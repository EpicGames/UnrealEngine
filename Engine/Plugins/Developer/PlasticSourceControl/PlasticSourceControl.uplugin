{
	"FileVersion": 3,
	"Version": 110,
	"VersionName": "1.11.0",
	"FriendlyName": "Plastic SCM",
	"Description": "Unity Version Control (formerly Plastic SCM)",
	"Category": "Source Control",
	"CreatedBy": "SRombauts for Codice Software.",
	"CreatedByURL": "https://srombauts.github.io",
	"DocsURL": "https://docs.unity.com/ugs/en-us/manual/devops/manual/vcs-plugins/unreal-plugin",
	"MarketplaceURL": "",
	"SupportURL": "https://support.unity.com/hc/en-us/requests/new?ticket_form_id=360001051792",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PlasticSourceControl",
			"Type": "UncookedOnly",
			"LoadingPhase": "EarliestPossible",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	]
}