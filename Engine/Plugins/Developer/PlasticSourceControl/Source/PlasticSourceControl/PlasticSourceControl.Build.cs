// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PlasticSourceControl : ModuleRules
{
	public PlasticSourceControl(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"UnrealEd",
				"LevelEditor",
				"DesktopPlatform",
				"SourceControl",
				"SourceControlWindows",
				"XmlParser",
				"Projects",
				"AssetRegistry",
				"DeveloperSettings",
				"ToolMenus",
				"ContentBrowser",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
