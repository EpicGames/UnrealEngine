// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPlasticSourceControlProvider;

class IPlasticSourceControlWorker
{
public:
	// Implemented in PlasticSourceControlOperations.cpp with all the code for the workers
	static void RegisterWorkers(FPlasticSourceControlProvider& PlasticSourceControlProvider);

public:
	explicit IPlasticSourceControlWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: PlasticSourceControlProvider(InSourceControlProvider)
	{
	}

	virtual ~IPlasticSourceControlWorker() = default;

	FPlasticSourceControlProvider& GetProvider()
	{
		return PlasticSourceControlProvider;
	}

	const FPlasticSourceControlProvider& GetProvider() const
	{
		return PlasticSourceControlProvider;
	}

	/**
	 * Name describing the work that this worker does. Used for factory method hookup.
	 */
	virtual FName GetName() const = 0;

	/**
	 * Function that actually does the work. Can be executed on another thread.
	 */
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) = 0;

	/**
	 * Updates the state of any items after completion (if necessary). This is always executed on the main thread.
	 * @returns true if states were updated
	 */
	virtual bool UpdateStates() = 0;

private:
	FPlasticSourceControlProvider& PlasticSourceControlProvider;
};

typedef TSharedRef<IPlasticSourceControlWorker, ESPMode::ThreadSafe> FPlasticSourceControlWorkerRef;
