// Copyright Epic Games, Inc. All Rights Reserved.

#include "Notification.h"

#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperationBase.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

// Display an ongoing notification during the whole operation
void FNotification::DisplayInProgress(const FText& InOperationInProgressString)
{
	if (!OperationInProgress.IsValid())
	{
		FNotificationInfo Info(InOperationInProgressString);
		Info.bFireAndForget = false;
		Info.ExpireDuration = 0.0f;
		Info.FadeOutDuration = 1.0f;
		OperationInProgress = FSlateNotificationManager::Get().AddNotification(Info);
		if (OperationInProgress.IsValid())
		{
			OperationInProgress.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
}

// Remove the ongoing notification at the end of the operation
void FNotification::RemoveInProgress()
{
	if (OperationInProgress.IsValid())
	{
		OperationInProgress.Pin()->ExpireAndFadeout();
		OperationInProgress.Reset();
	}
}

// Add or Queue a notification and dispose of the allocated memory if necessary
static void AddOrQueueNotification(FNotificationInfo* Info)
{
	// AddNotification must be called on game thread. Use QueueNotification if necessary.
	// Note: not using QueueNotification if not necessary since it alter the order of notifications when mix with In Progress notifications.
	if (IsInGameThread())
	{
		FSlateNotificationManager::Get().AddNotification(*Info);
		delete Info;
	}
	else
	{
		FSlateNotificationManager::Get().QueueNotification(Info);
	}
}

void FNotification::DisplayResult(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	DisplayResult(StaticCastSharedRef<FSourceControlOperationBase>(InOperation).Get(), InResult);
}

void FNotification::DisplayResult(const FSourceControlOperationBase& InOperation, ECommandResult::Type InResult)
{
	if (InResult == ECommandResult::Succeeded)
	{
		FNotification::DisplaySuccess(InOperation);
	}
	else
	{
		FNotification::DisplayFailure(InOperation);
	}
}

// Display a temporary success notification at the end of the operation
void FNotification::DisplaySuccess(const FSourceControlOperationBase& InOperation)
{
	if (InOperation.GetResultInfo().InfoMessages.Num() > 0)
	{
		// If there are multiple messages, display the last one to not let the user with a notification starting with a "wait" or "in progress" message
		DisplaySuccess(InOperation.GetResultInfo().InfoMessages.Last());
	}
	else
	{
		const FText NotificationText = FText::Format(
			LOCTEXT("PlasticSourceControlOperation_Success", "{0} operation was successful."),
			FText::FromName(InOperation.GetName())
		);

		DisplaySuccess(NotificationText);
	}
}

void FNotification::DisplaySuccess(const FText& InNotificationText)
{
	FNotificationInfo* Info = new FNotificationInfo(InNotificationText);
	Info->ExpireDuration = 3.0f;
	Info->bFireAndForget = true;
	Info->bUseSuccessFailIcons = true;
	Info->Image = FAppStyle::GetBrush(TEXT("Icons.SuccessWithColor.Large"));
	AddOrQueueNotification(Info);

	UE_LOGF(LogSourceControl, Verbose, "%ls", *InNotificationText.ToString());
}

// Display a temporary failure notification at the end of the operation
void FNotification::DisplayFailure(const FSourceControlOperationBase& InOperation)
{
	if (InOperation.GetResultInfo().ErrorMessages.Num() > 0)
	{
		// If there are multiple messages, display the last one to not let the user with a notification starting with a "wait" or "in progress" message
		const FText NotificationText = FText::Format(
			LOCTEXT("PlasticSourceControlOperation_FailureWithMessages", "Error: {0} operation failed!\n{1}"),
			FText::FromName(InOperation.GetName()),
			InOperation.GetResultInfo().ErrorMessages.Last()
		);

		DisplayFailure(NotificationText);
	}
	else
	{
		const FText NotificationText = FText::Format(
			LOCTEXT("PlasticSourceControlOperation_Failure", "Error: {0} operation failed!"),
			FText::FromName(InOperation.GetName())
		);

		DisplayFailure(NotificationText);
	}
}

void FNotification::DisplayFailure(const FText& InNotificationText)
{
	FNotificationInfo* Info = new FNotificationInfo(InNotificationText);
	Info->ExpireDuration = 10.0f;
	Info->bFireAndForget = true;
	Info->bUseSuccessFailIcons = true;
	Info->Image = FAppStyle::GetBrush(TEXT("Icons.ErrorWithColor.Large"));
	// Provide a link to easily open the Output Log
	Info->Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
	Info->HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");
	AddOrQueueNotification(Info);

	UE_LOGF(LogSourceControl, Error, "%ls", *InNotificationText.ToString());
}

#undef LOCTEXT_NAMESPACE
