// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlProvider.h"

class ISourceControlOperation;
class FSourceControlOperationBase;

class FNotification
{
public:
	// Create and display (resp. expire and remove) an in-progress notification for a long-running operation
	// Note: UI main thread only
	void DisplayInProgress(const FText& InOperationInProgressString);
	void RemoveInProgress();

	bool IsInProgress() const
	{
		return OperationInProgress.IsValid();
	}

	// Display a short fire-and-forget notification
	// Note: thread safe methods of queuing a notification from any thread
	static void DisplayResult(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	static void DisplayResult(const FSourceControlOperationBase& InOperation, ECommandResult::Type InResult);

	static void DisplaySuccess(const FSourceControlOperationBase& InOperation);
	static void DisplaySuccess(const FText& InNotificationText);

	static void DisplayFailure(const FSourceControlOperationBase& InOperation);
	static void DisplayFailure(const FText& InNotificationText);

private:
	/** Current long-running notification if any */
	TWeakPtr<class SNotificationItem> OperationInProgress;
};
