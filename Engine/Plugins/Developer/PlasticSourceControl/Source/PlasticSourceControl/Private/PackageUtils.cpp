// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageUtils.h"

#include "Async/Async.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h" // LogSourceControl
#include "Misc/Paths.h"
#include "PackageTools.h"
#include "UObject/Linker.h"

namespace PackageUtils
{

// Get the World currently loaded by the Editor (and thus, access to the corresponding map package)
static UWorld* GetCurrentWorld()
{
	if (GEditor)
	{
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			return EditorWorld;
		}
	}
	return nullptr;
}

TArray<FString> AssetDataToFileNames(const TArray<FAssetData>& InAssetObjectPaths)
{
	TArray<FString> FileNames;
	FileNames.Reserve(InAssetObjectPaths.Num());
	for (const FAssetData& AssetObjectPath : InAssetObjectPaths)
	{
		//  Tries to find the package in memory if it is loaded, otherwise loads it
		if (UPackage* Package = AssetObjectPath.GetPackage())
		{
			const FString PackageExtension = Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			FString PackageFilename = FPackageName::LongPackageNameToFilename(Package->GetName(), PackageExtension);
			FileNames.Add(MoveTemp(PackageFilename));
		}
	}
	return FileNames;
}

/// Prompt to save or discard all packages
bool SaveDirtyPackages()
{
	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = true; // If the user clicks "don't save" this will continue and lose their changes
	bool bHadPackagesToSave = false;

	bool bSaved = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, &bHadPackagesToSave);

	// bSaved can be true if the user selects to not save an asset by un-checking it and clicking "save"
	if (bSaved)
	{
		bSaved = !FEditorFileUtils::HasDirtyPackages();
	}

	return bSaved;
}

/// Find all packages in Content directory
TArray<FString> ListAllPackages()
{
	TArray<FString> PackageFilePaths;
	FPackageName::FindPackagesInDirectory(PackageFilePaths, FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
	return PackageFilePaths;
}

// Find the packages corresponding to the files, if they are loaded in memory (won't load them)
// Note: Extracted from AssetViewUtils::SyncPathsFromSourceControl()
static TArray<UPackage*> FileNamesToLoadedPackages(const TArray<FString>& InFiles)
{
	TArray<UPackage*> LoadedPackages;
	LoadedPackages.Reserve(InFiles.Num() + 1);
	for (const FString& FilePath : InFiles)
	{
		FString PackageName;
		FString FailureReason;
		if (FPackageName::TryConvertFilenameToLongPackageName(FilePath, PackageName, &FailureReason))
		{
			// NOTE: this will only find packages loaded in memory
			if (UPackage* Package = FindPackage(nullptr, *PackageName))
			{
				LoadedPackages.Emplace(Package);
			}
		}
		// else, it means the file is not an asset from the Content/ folder (eg config, source code, anything else)
	}
	return LoadedPackages;
}

// Unlink all loaded packages to allow to update them
// Note: Extracted from AssetViewUtils::SyncPathsFromSourceControl()
void UnlinkPackages(const TArray<FString>& InFiles)
{
	const TArray<UPackage*> LoadedPackages = FileNamesToLoadedPackages(InFiles);
	for (UPackage* Package : LoadedPackages)
	{
		// Detach the linkers of any loaded packages so that SCC can overwrite the files...
		if (!Package->IsFullyLoaded())
		{
			FlushAsyncLoading();
			Package->FullyLoad();
		}
		ResetLoaders(Package);
	}
	if (LoadedPackages.Num() > 0)
	{
		UE_LOGF(LogSourceControl, Log, "Reseted Loader for %d Packages", LoadedPackages.Num());
	}
}

// Uses AsyncTask to call UnlinkPackages() on the Game Thread, and use a Promise to wait for the operation to complete
void UnlinkPackagesInMainThread(const TArray<FString>& InFiles)
{
	const TSharedRef<TPromise<void>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<void>());
	AsyncTask(ENamedThreads::GameThread, [Promise, InFiles]()
	{
		PackageUtils::UnlinkPackages(InFiles);
		Promise->SetValue();
	});
	Promise->GetFuture().Get();
}

static TArray<UPackage*> ListPackagesToReload(const TArray<FString>& InFiles)
{
	TArray<UPackage*> LoadedPackages = FileNamesToLoadedPackages(InFiles);

	// Detects if some packages to reload are part of the current map
	// (ie assets within __ExternalActors__ or __ExternalObjects__ from the new One File Per Actor (OFPA) in UE5)
	// in which case the current map need to be reloaded, so it needs to be added to the list of packages if not already there
	// (then UPackageTools::ReloadPackages() will handle unloading the map at the start of the reload, avoiding some crash, and reloading it at the end)
	if (UWorld* CurrentWorld = GetCurrentWorld())
	{
		UPackage* CurrentMapPackage = CurrentWorld->GetOutermost();

		// If the current map file has been updated, it will be reloaded automatically, so no need for the following
		const FString CurrentMapFileAbsolute = FPaths::ConvertRelativePathToFull(CurrentMapPackage->GetLoadedPath().GetLocalFullPath());
		const bool bHasCurrentMapBeenUpdated = InFiles.FindByPredicate(
			[&CurrentMapFileAbsolute](const FString& InFilePath) { return InFilePath.Equals(CurrentMapFileAbsolute, ESearchCase::IgnoreCase); }
		) != nullptr;

		if (!bHasCurrentMapBeenUpdated)
		{
			static const FString GamePath = FString("/Game");
			const FString CurrentMapPath = *CurrentMapPackage->GetName();																	// eg "/Game/Maps/OpenWorld"
			const FString CurrentMapPathWithoutGamePrefix = CurrentMapPath.RightChop(GamePath.Len());										// eg "/Maps/OpenWorld"
			const FString CurrentMapExternalActorPath = FPackagePath::GetExternalActorsFolderName() + CurrentMapPathWithoutGamePrefix;		// eg "/__ExternalActors__/Maps/OpenWorld"
			const FString CurrentMapExternalObjectPath = FPackagePath::GetExternalObjectsFolderName() + CurrentMapPathWithoutGamePrefix;	// eg "/__ExternalObjects__/Maps/OpenWorld"

			bool bNeedReloadCurrentMap = false;

			for (const FString& FilePath : InFiles)
			{
				if (FilePath.Contains(CurrentMapExternalActorPath) || FilePath.Contains(CurrentMapExternalObjectPath))
				{
					bNeedReloadCurrentMap = true;
					break;
				}
			}

			if (bNeedReloadCurrentMap)
			{
				LoadedPackages.Add(CurrentMapPackage);
				UE_LOGF(LogSourceControl, Log, "Reload: %ls", *CurrentMapPath);
			}
		}
	}

	return LoadedPackages;
}

// Hot-Reload all packages after they have been updated
// Note: Extracted from AssetViewUtils::SyncPathsFromSourceControl()
void ReloadPackages(TArray<UPackage*>& InPackages)
{
	UE_LOGF(LogSourceControl, Verbose, "Reloading %d Packages...", InPackages.Num());

	// Syncing may have deleted some packages, so we need to unload those rather than re-load them...
	// Note: we will store the package using weak pointers here otherwise we might have garbage collection issues after the ReloadPackages call
	TArray<TWeakObjectPtr<UPackage>> PackagesToUnload;
	InPackages.RemoveAll([&](UPackage* InPackage) -> bool
	{
		const FString PackageExtension = InPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
		const FString PackageFilename = FPackageName::LongPackageNameToFilename(InPackage->GetName(), PackageExtension);
		if (!FPaths::FileExists(PackageFilename))
		{
			PackagesToUnload.Emplace(MakeWeakObjectPtr(InPackage));
			return true; // remove package
		}
		return false; // keep package
	});

	// Hot-reload the new packages...
	UPackageTools::ReloadPackages(InPackages);

	// Unload any deleted packages...
	TArray<UPackage*> PackageRawPtrsToUnload;
	for (TWeakObjectPtr<UPackage>& PackageToUnload : PackagesToUnload)
	{
		if (PackageToUnload.IsValid())
		{
			PackageRawPtrsToUnload.Emplace(PackageToUnload.Get());
		}
	}

	UPackageTools::UnloadPackages(PackageRawPtrsToUnload);
}

// Reload packages that where updated by the operation (and the current map if needed)
void ReloadPackages(const TArray<FString>& InFiles)
{
	TArray<UPackage*> PackagesToReload = ListPackagesToReload(InFiles);
	if (PackagesToReload.Num() > 0)
	{
		ReloadPackages(PackagesToReload);
	}
	else
	{
		UE_LOGF(LogSourceControl, Verbose, "No package to reload");
	}
}

void ReloadPackagesInMainThread(const TArray<FString>& InFiles)
{
	TSharedRef<TPromise<void>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<void>());
	AsyncTask(ENamedThreads::GameThread, [Promise, InFiles]()
	{
		PackageUtils::ReloadPackages(InFiles);
		Promise->SetValue();
	});
	Promise->GetFuture().Get();
}

} // namespace PackageUtils
