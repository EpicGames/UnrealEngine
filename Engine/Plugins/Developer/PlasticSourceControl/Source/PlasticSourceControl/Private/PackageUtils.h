// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"

namespace PackageUtils
{
	TArray<FString> AssetDataToFileNames(const TArray<FAssetData>& InAssetObjectPaths);

	bool SaveDirtyPackages();
	TArray<FString> ListAllPackages();

	void UnlinkPackages(const TArray<FString>& InFiles);
	void UnlinkPackagesInMainThread(const TArray<FString>& InFiles);

	void ReloadPackages(const TArray<FString>& InFiles);
	void ReloadPackagesInMainThread(const TArray<FString>& InFiles);
} // namespace PackageUtils
