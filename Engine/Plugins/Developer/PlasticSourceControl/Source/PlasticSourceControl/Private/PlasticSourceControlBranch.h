// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPlasticSourceControlBranch
{
public:
	FString Name;
	FString Repository;
	FString CreatedBy;
	FDateTime Date;
	FString Comment;

	void PopulateSearchString(TArray<FString>& OutStrings) const
	{
		OutStrings.Emplace(Name);
		OutStrings.Emplace(CreatedBy);
		OutStrings.Emplace(Comment);
	}
};

typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;
typedef TSharedPtr<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchPtr;
