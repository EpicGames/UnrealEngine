// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlBranchesWindow.h"

#include "SPlasticSourceControlBranchesWidget.h"

#include "Framework/Docking/TabManager.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

static const FName PlasticSourceControlBranchesWindowTabName("PlasticSourceControlBranchesWindow");

void FPlasticSourceControlBranchesWindow::Register()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PlasticSourceControlBranchesWindowTabName, FOnSpawnTab::CreateRaw(this, &FPlasticSourceControlBranchesWindow::OnSpawnTab))
		.SetDisplayName(LOCTEXT("PlasticSourceControlBranchesWindowTabTitle", "View Branches"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Branched"));
}

void FPlasticSourceControlBranchesWindow::Unregister()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PlasticSourceControlBranchesWindowTabName);
}

TSharedRef<SDockTab> FPlasticSourceControlBranchesWindow::OnSpawnTab(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateBranchesWidget().ToSharedRef()
		];
}

void FPlasticSourceControlBranchesWindow::OpenTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PlasticSourceControlBranchesWindowTabName);
}

TSharedPtr<SWidget> FPlasticSourceControlBranchesWindow::CreateBranchesWidget()
{
	return SNew(SPlasticSourceControlBranchesWidget);
}

#undef LOCTEXT_NAMESPACE
