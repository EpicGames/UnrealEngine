// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Nomad tab window to hold the widget with the list of branches, see SPlasticSourceControlBranchesWidget
class FPlasticSourceControlBranchesWindow
{
public:
	void Register();
	void Unregister();

	void OpenTab();

private:
	TSharedRef<class SDockTab> OnSpawnTab(const class FSpawnTabArgs& SpawnTabArgs);

	TSharedPtr<class SWidget> CreateBranchesWidget();
};
