// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlChangelist.h"

const FPlasticSourceControlChangelist FPlasticSourceControlChangelist::DefaultChangelist(TEXT("Default"), true);
