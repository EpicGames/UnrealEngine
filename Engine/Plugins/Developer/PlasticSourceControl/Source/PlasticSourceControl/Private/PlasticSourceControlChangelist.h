// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlChangelist.h"

/**
 * Unique Identifier of a changelist under source control: a "name" in Unity Version Control
 */
class FPlasticSourceControlChangelist : public ISourceControlChangelist
{
public:
	FPlasticSourceControlChangelist() = default;

	explicit FPlasticSourceControlChangelist(FString&& InChangelistName, const bool bInInitialized = false)
		: ChangelistName(MoveTemp(InChangelistName))
		, bInitialized(bInInitialized)
	{
	}

	virtual bool CanDelete() const override
	{
		return !IsDefault();
	}

	bool operator==(const FPlasticSourceControlChangelist& InOther) const
	{
		return ChangelistName == InOther.ChangelistName;
	}

	bool operator!=(const FPlasticSourceControlChangelist& InOther) const
	{
		return ChangelistName != InOther.ChangelistName;
	}

	virtual bool IsDefault() const override
	{
		return ChangelistName == DefaultChangelist.ChangelistName;
	}

	void SetInitialized()
	{
		bInitialized = true;
	}

	bool IsInitialized() const
	{
		return bInitialized;
	}

	void Reset()
	{
		ChangelistName.Reset();
		bInitialized = false;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FPlasticSourceControlChangelist& InPlasticChangelist)
	{
		return GetTypeHash(InPlasticChangelist.ChangelistName);
	}

	FString GetName() const
	{
		return ChangelistName;
	}

	virtual FString GetIdentifier() const override
	{
		return ChangelistName;
	}

public:
	static const FPlasticSourceControlChangelist DefaultChangelist;

private:
	FString ChangelistName;
	bool bInitialized = false;
};

typedef TSharedRef<class FPlasticSourceControlChangelist, ESPMode::ThreadSafe> FPlasticSourceControlChangelistRef;
