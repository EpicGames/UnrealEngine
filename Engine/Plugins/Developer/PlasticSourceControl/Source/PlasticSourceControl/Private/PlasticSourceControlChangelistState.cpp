// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlChangelistState.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl.ChangelistState"

FName FPlasticSourceControlChangelistState::GetIconName() const
{
	// Mimic P4V colors, returning the red icon if there are active file(s), the blue if the changelist is empty or all the files are shelved.
	return Files.Num() > 0 ? FName("SourceControl.Changelist") : FName("SourceControl.ShelvedCHangelist");
}

FName FPlasticSourceControlChangelistState::GetSmallIconName() const
{
	return GetIconName();
}

FText FPlasticSourceControlChangelistState::GetDisplayText() const
{
	return FText::FromString(Changelist.GetName());
}

FText FPlasticSourceControlChangelistState::GetDescriptionText() const
{
	return FText::FromString(Description);
}

FText FPlasticSourceControlChangelistState::GetDisplayTooltip() const
{
	return LOCTEXT("Tooltip", "Tooltip");
}

const FDateTime& FPlasticSourceControlChangelistState::GetTimeStamp() const
{
	return TimeStamp;
}

const TArray<FSourceControlStateRef> FPlasticSourceControlChangelistState::GetFilesStates() const
{
	return Files;
}

int32 FPlasticSourceControlChangelistState::GetFilesStatesNum() const
{
	return Files.Num();
}

const TArray<FSourceControlStateRef> FPlasticSourceControlChangelistState::GetShelvedFilesStates() const
{
	return ShelvedFiles;
}

int32 FPlasticSourceControlChangelistState::GetShelvedFilesStatesNum() const
{
	return ShelvedFiles.Num();
}

FSourceControlChangelistRef FPlasticSourceControlChangelistState::GetChangelist() const
{
	FPlasticSourceControlChangelistRef ChangelistCopy = MakeShareable( new FPlasticSourceControlChangelist(Changelist));
	return StaticCastSharedRef<ISourceControlChangelist>(ChangelistCopy);
}

#undef LOCTEXT_NAMESPACE
