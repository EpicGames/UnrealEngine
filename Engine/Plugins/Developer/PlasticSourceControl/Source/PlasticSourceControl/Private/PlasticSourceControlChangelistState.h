// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlChangelistState.h"
#include "ISourceControlState.h"

#include "PlasticSourceControlChangelist.h"

/**
 * The state of a pending changelist under source control: description and list of files
 */
class FPlasticSourceControlChangelistState : public ISourceControlChangelistState
{
public:
	explicit FPlasticSourceControlChangelistState(const FPlasticSourceControlChangelist& InChangelist, const FString& InDescription = FString())
		: Changelist(InChangelist)
		, Description(InDescription)
	{
	}

	explicit FPlasticSourceControlChangelistState(FPlasticSourceControlChangelist&& InChangelist, FString&& InDescription)
		: Changelist(MoveTemp(InChangelist))
		, Description(MoveTemp(InDescription))
	{
	}

	/**
	 * Get the name of the icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetIconName() const override;

	/**
	 * Get the name of the small icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetSmallIconName() const override;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDisplayText() const override;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDescriptionText() const override;

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	virtual FText GetDisplayTooltip() const override;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	virtual const FDateTime& GetTimeStamp() const override;

	virtual const TArray<FSourceControlStateRef> GetFilesStates() const override;
	virtual int32 GetFilesStatesNum() const override;

	virtual const TArray<FSourceControlStateRef> GetShelvedFilesStates() const override;
	virtual int32 GetShelvedFilesStatesNum() const override;

	virtual FSourceControlChangelistRef GetChangelist() const override;

public:
	FPlasticSourceControlChangelist Changelist;

	FString Description;

	TArray<FSourceControlStateRef> Files;

	int32 ShelveId = ISourceControlState::INVALID_REVISION;
	FDateTime ShelveDate;

	TArray<FSourceControlStateRef> ShelvedFiles;

	/** The timestamp of the last update */
	FDateTime TimeStamp;
};
