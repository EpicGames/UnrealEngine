// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PlasticSourceControlState.h"

class FPlasticSourceControlChangeset
{
public:
	int32 ChangesetId;
	FString CreatedBy;
	FDateTime Date;
	FString Comment;
	FString Branch;
	// Note: array of file States, each with one Revision for Diffing (like for Files and ShelvedFiles in FPlasticSourceControlChangelist)
	TArray<FPlasticSourceControlStateRef> Files;

	void PopulateSearchString(TArray<FString>& OutStrings) const
	{
		OutStrings.Emplace(CreatedBy);
		OutStrings.Emplace(Comment);
		OutStrings.Emplace(Branch);
	}
};

typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedPtr<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetPtr;
