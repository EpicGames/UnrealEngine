// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlChangesetsWindow.h"

#include "SPlasticSourceControlChangesetsWidget.h"

#include "Framework/Docking/TabManager.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Widgets/Docking/SDockTab.h"


#define LOCTEXT_NAMESPACE "PlasticSourceControlChangesetsWindow"

static const FName PlasticSourceControlChangesetsWindowTabName("PlasticSourceControlChangesetsWindow");

void FPlasticSourceControlChangesetsWindow::Register()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PlasticSourceControlChangesetsWindowTabName, FOnSpawnTab::CreateRaw(this, &FPlasticSourceControlChangesetsWindow::OnSpawnTab))
		.SetDisplayName(LOCTEXT("PlasticSourceControlChangesetsWindowTabTitle", "View Changesets"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.History"));
}

void FPlasticSourceControlChangesetsWindow::Unregister()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PlasticSourceControlChangesetsWindowTabName);
}

TSharedRef<SDockTab> FPlasticSourceControlChangesetsWindow::OnSpawnTab(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateChangesetsWidget().ToSharedRef()
		];
}

void FPlasticSourceControlChangesetsWindow::OpenTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PlasticSourceControlChangesetsWindowTabName);
}

TSharedPtr<SWidget> FPlasticSourceControlChangesetsWindow::CreateChangesetsWidget()
{
	return SNew(SPlasticSourceControlChangesetsWidget);
}

#undef LOCTEXT_NAMESPACE
