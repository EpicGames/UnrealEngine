// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlCommand.h"

#include "PlasticSourceControlModule.h"

#include "ISourceControlOperation.h"
#include "HAL/PlatformTime.h"

FPlasticSourceControlCommand::FPlasticSourceControlCommand(const FSourceControlOperationRef& InOperation, const FPlasticSourceControlWorkerRef& InWorker, const FSourceControlOperationComplete& InOperationCompleteDelegate)
	: Operation(InOperation)
	, Worker(InWorker)
	, OperationCompleteDelegate(InOperationCompleteDelegate)
	, bExecuteProcessed(0)
	, bCommandSuccessful(false)
	, bConnectionDropped(false)
	, bAutoDelete(true)
	, Concurrency(EConcurrency::Synchronous)
	, StartTimestamp(FPlatformTime::Seconds())
{
	// grab the providers settings here, so we don't access them once the worker thread is launched
	check(IsInGameThread());
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	PathToWorkspaceRoot = Provider.GetPathToWorkspaceRoot();
	ChangesetNumber = Provider.GetChangesetNumber();
}

bool FPlasticSourceControlCommand::DoWork()
{
	bCommandSuccessful = Worker->Execute(*this);
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);

	return bCommandSuccessful;
}

void FPlasticSourceControlCommand::Abandon()
{
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);
}

void FPlasticSourceControlCommand::DoThreadedWork()
{
	Concurrency = EConcurrency::Asynchronous;
	DoWork();
}

ECommandResult::Type FPlasticSourceControlCommand::ReturnResults()
{
	// Save any messages that have accumulated
	for (FString& String : InfoMessages)
	{
		Operation->AddInfoMessge(FText::FromString(String));
	}
	for (FString& String : ErrorMessages)
	{
		Operation->AddErrorMessge(FText::FromString(String));
	}

	// run the completion delegate if we have one bound
	ECommandResult::Type Result = bCommandSuccessful ? ECommandResult::Succeeded : ECommandResult::Failed;
	OperationCompleteDelegate.ExecuteIfBound(Operation, Result);

	return Result;
}
