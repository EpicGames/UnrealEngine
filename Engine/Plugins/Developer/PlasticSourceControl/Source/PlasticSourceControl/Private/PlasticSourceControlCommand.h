// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/IQueuedWork.h"
#include "ISourceControlProvider.h"
#include "IPlasticSourceControlWorker.h"

#include "PlasticSourceControlChangelist.h"

/**
 * Used to execute Plastic commands multi-threaded.
 */
class FPlasticSourceControlCommand : public IQueuedWork
{
public:
	FPlasticSourceControlCommand(const FSourceControlOperationRef& InOperation, const FPlasticSourceControlWorkerRef& InWorker, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * This is where the real thread work is done. All work that is done for
	 * this queued object should be done from within the call to this function.
	 */
	bool DoWork();

	/**
	 * Tells the queued work that it is being abandoned so that it can do
	 * per object clean up as needed. This will only be called if it is being
	 * abandoned before completion. NOTE: This requires the object to delete
	 * itself using whatever heap it was allocated in.
	 */
	virtual void Abandon() override;

	/**
	 * This method is also used to tell the object to cleanup but not before
	 * the object has finished it's work.
	 */
	virtual void DoThreadedWork() override;

	/** Save any results and call any registered callbacks. */
	ECommandResult::Type ReturnResults();

public:
	/** Path to the root of the Plastic workspace: can be the GameDir itself, or any parent directory (found by the "Connect" operation) */
	FString PathToWorkspaceRoot;

	/** Operation we want to perform - contains outward-facing parameters & results */
	FSourceControlOperationRef Operation;

	/** The object that will actually do the work */
	FPlasticSourceControlWorkerRef Worker;

	/** Delegate to notify when this operation completes */
	FSourceControlOperationComplete OperationCompleteDelegate;

	/**If true, this command has been processed by the source control thread*/
	volatile int32 bExecuteProcessed;

	/**If true, the source control command succeeded*/
	bool bCommandSuccessful;

	/**If true, the source control connection was dropped while this command was being executed*/
	bool bConnectionDropped;

	/** Plastic current workspace */
	FString WorkspaceName;

	/** Plastic current repository */
	FString RepositoryName;

	/** Plastic current server URL */
	FString ServerUrl;

	/** Name of the current branch */
	FString BranchName;

	/** Name of the object that the workspace is switched to: usually the branch, sometimes a changeset or a label */
	FString WorkspaceSelector;

	/** Current Changeset Number */
	int32 ChangesetNumber;

	/** If true, this command will be automatically cleaned up in Tick() */
	bool bAutoDelete;

	/** Whether we are running multi-treaded in the background, or blocking the main thread */
	EConcurrency::Type Concurrency;

	/** Timestamp of when the command was issued */
	const double StartTimestamp;

	/** Files to perform this operation on */
	TArray<FString> Files;

	/** Changelist to perform this operation on */
	FPlasticSourceControlChangelist Changelist;

	/**Info and/or warning message storage*/
	TArray<FString> InfoMessages;

	/**Potential error message storage*/
	TArray<FString> ErrorMessages;
};
