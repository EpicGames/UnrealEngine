// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlConsole.h"

#include "PlasticSourceControlUtils.h"

#include "ISourceControlModule.h"

void FPlasticSourceControlConsole::Register()
{
	if (!CmConsoleCommand.IsValid())
	{
		CmConsoleCommand = MakeUnique<FAutoConsoleCommand>(
			TEXT("cm"),
			TEXT("Unity Version Control (formerly Plastic SCM) Command Line Interface (cm).\n")
			TEXT("Run any 'cm' command directly from the Unreal Editor Console.\n")
			TEXT("Type 'cm showcommands' to get a command list."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FPlasticSourceControlConsole::ExecutePlasticConsoleCommand));
	}
}

void FPlasticSourceControlConsole::Unregister()
{
	CmConsoleCommand.Reset();
}

void FPlasticSourceControlConsole::ExecutePlasticConsoleCommand(const TArray<FString>& a_args)
{
	// If called with no argument, explicitly call "cm help" instead to mimic the cm CLI behavior.
	if (a_args.Num() < 1)
	{
		ExecutePlasticConsoleCommand(TArray<FString>({TEXT("help")}));
		return;
	}

	FString Results;
	FString Errors;
	const FString Command = a_args[0];
	TArray<FString> Parameters = a_args;
	Parameters.RemoveAt(0);
	PlasticSourceControlUtils::RunCommand(Command, Parameters, TArray<FString>(), Results, Errors);
	if (Results.Len() > 200) // RunCommand() already log all command results up to 200 characters (limit to avoid long wall of text like XML)
	{
		UE_LOGF(LogSourceControl, Log, "Output:\n%ls", *Results);
	}
}
