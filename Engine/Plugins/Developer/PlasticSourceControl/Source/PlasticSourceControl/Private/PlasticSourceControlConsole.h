// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

/**
 * Editor only console commands.
 *
 * Such commands can be executed from the editor output log window, but also from command line arguments,
 * from Editor Blueprints utilities, or from C++ Code using. eg. GEngine->Exec("cm status");
 */
class FPlasticSourceControlConsole
{
public:
	void Register();
	void Unregister();

private:
	// Unity Version Control Command Line Interface: Run 'cm' commands directly from the Unreal Editor Console.
	void ExecutePlasticConsoleCommand(const TArray<FString>& a_args);

	/** Console command for interacting with 'cm' CLI directly */
	TUniquePtr<FAutoConsoleCommand> CmConsoleCommand;
};
