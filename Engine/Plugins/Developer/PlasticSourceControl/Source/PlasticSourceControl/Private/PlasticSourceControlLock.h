// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlState.h"

class FPlasticSourceControlLock
{
public:
	int32 ItemId = ISourceControlState::INVALID_REVISION;
	FString Path;
	FString Status;
	bool bIsLocked = false;
	FDateTime Date;
	FString Owner;
	FString DestinationBranch;
	FString Branch;
	FString Workspace;

	void PopulateSearchString(TArray<FString>& OutStrings) const
	{
		OutStrings.Emplace(Path);
		OutStrings.Emplace(Owner);
		OutStrings.Emplace(Branch);
		OutStrings.Emplace(Workspace);
	}
};

typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;
typedef TSharedPtr<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockPtr;
