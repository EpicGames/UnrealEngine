// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlLocksWindow.h"

#include "SPlasticSourceControlLocksWidget.h"

#include "Framework/Docking/TabManager.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlLocksWindow"

static const FName PlasticSourceControlLocksWindowTabName("PlasticSourceControlLocksWindow");

void FPlasticSourceControlLocksWindow::Register()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PlasticSourceControlLocksWindowTabName, FOnSpawnTab::CreateRaw(this, &FPlasticSourceControlLocksWindow::OnSpawnTab))
		.SetDisplayName(LOCTEXT("PlasticSourceControlLocksWindowTabTitle", "View Locks"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Locked"));
}

void FPlasticSourceControlLocksWindow::Unregister()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PlasticSourceControlLocksWindowTabName);
}

TSharedRef<SDockTab> FPlasticSourceControlLocksWindow::OnSpawnTab(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateLocksWidget().ToSharedRef()
		];
}

void FPlasticSourceControlLocksWindow::OpenTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PlasticSourceControlLocksWindowTabName);
}

TSharedPtr<SWidget> FPlasticSourceControlLocksWindow::CreateLocksWidget()
{
	return SNew(SPlasticSourceControlLocksWidget);
}

#undef LOCTEXT_NAMESPACE
