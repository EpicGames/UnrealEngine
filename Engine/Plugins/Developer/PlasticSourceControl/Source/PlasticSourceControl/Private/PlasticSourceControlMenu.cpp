// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlMenu.h"

#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlUtils.h"
#include "PlasticSourceControlVersions.h"
#include "SPlasticSourceControlStatusBar.h"

#include "ISourceControlModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"

#include "ContentBrowserMenuContexts.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformProcess.h"
#include "LevelEditor.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"

#include "PackageUtils.h"
#include "ISettingsModule.h"

#include "Logging/MessageLog.h"

#include "ToolMenus.h"
#include "ToolMenuMisc.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

FName FPlasticSourceControlMenu::UnityVersionControlMainMenuOwnerName = TEXT("UnityVersionControlMenu");
FName FPlasticSourceControlMenu::UnityVersionControlAssetContextLocksMenuOwnerName = TEXT("UnityVersionControlContextLocksMenu");
FName FPlasticSourceControlMenu::UnityVersionControlStatusBarMenuOwnerName = TEXT("UnityVersionControlStatusBarMenu");

void FPlasticSourceControlMenu::Register()
{
	if (bHasRegistered)
	{
		return;
	}

	// Register the menu extensions with the level editor
	ExtendRevisionControlMenu();
	ExtendAssetContextMenu();

	ExtendToolbarWithStatusBarWidget();
}

void FPlasticSourceControlMenu::Unregister()
{
	if (!bHasRegistered)
	{
		return;
	}

	// Unregister the menu extensions from the level editor
	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		ToolMenus->UnregisterOwnerByName(UnityVersionControlMainMenuOwnerName);
		ToolMenus->UnregisterOwnerByName(UnityVersionControlAssetContextLocksMenuOwnerName);
		ToolMenus->UnregisterOwnerByName(UnityVersionControlStatusBarMenuOwnerName);
		bHasRegistered = false;
	}
}

void FPlasticSourceControlMenu::ExtendToolbarWithStatusBarWidget()
{
	const FToolMenuOwnerScoped SourceControlMenuOwner(UnityVersionControlStatusBarMenuOwnerName);

	UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.StatusBar.ToolBar");
	FToolMenuSection& Section = ToolbarMenu->AddSection("Unity Version Control", FText::GetEmpty(), FToolMenuInsert("SourceControl", EToolMenuInsertType::Before));

	Section.AddEntry(
		FToolMenuEntry::InitWidget("UnityVersionControlStatusBar", SNew(SPlasticSourceControlStatusBar), FText::GetEmpty(), true, false)
	);
}

void FPlasticSourceControlMenu::ExtendRevisionControlMenu()
{
	const FToolMenuOwnerScoped SourceControlMenuOwner(UnityVersionControlMainMenuOwnerName);

	if (UToolMenu* SourceControlMenu = UToolMenus::Get()->ExtendMenu("StatusBar.ToolBar.SourceControl"))
	{
		FToolMenuSection& Section = SourceControlMenu->AddSection("PlasticSourceControlActions", LOCTEXT("PlasticSourceControlMenuHeadingActions", "Unity Version Control"), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

		AddMenuExtension(Section);

		bHasRegistered = true;
	}

	if (UToolMenu* ToolsMenu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Tools"))
	{
		if (FToolMenuSection* Section = ToolsMenu->FindSection("Source Control"))
		{
			AddViewBranches(*Section);
			AddViewChangesets(*Section);
			AddViewLocks(*Section);
		}
	}
}

void FPlasticSourceControlMenu::ExtendAssetContextMenu()
{
	const FToolMenuOwnerScoped SourceControlMenuOwner(UnityVersionControlAssetContextLocksMenuOwnerName);
	if (UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(TEXT("ContentBrowser.AssetContextMenu")))
	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("PlasticAssetContextLocksMenuSection"), FText::GetEmpty(), FToolMenuInsert("AssetContextReferences", EToolMenuInsertType::After));
		Section.AddDynamicEntry(TEXT("PlasticActions"), FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();

			if (!Context || !Context->bCanBeModified || Context->SelectedAssets.Num() == 0 || !ensure(FPlasticSourceControlModule::IsLoaded()))
			{
				return;
			}
			TArray<FAssetData> AssetObjectPaths = Context->SelectedAssets;

			InSection.AddSubMenu(
				TEXT("PlasticActionsSubMenu"),
				LOCTEXT("Plastic_ContextMenu", "Revision Control Locks"),
				FText::GetEmpty(),
				FNewMenuDelegate::CreateRaw(this, &FPlasticSourceControlMenu::GeneratePlasticAssetContextMenu, MoveTemp(AssetObjectPaths)),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Locked")
			);
		}));
	}
}

void FPlasticSourceControlMenu::GeneratePlasticAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> InAssetObjectPaths)
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const TArray<FString> Files = PackageUtils::AssetDataToFileNames(InAssetObjectPaths);
	const TArray<FPlasticSourceControlLockRef> SelectedLocks = PlasticSourceControlUtils::GetLocksForWorkingBranch(Provider, Files);

	MenuBuilder.BeginSection("AssetPlasticActions", LOCTEXT("UnityVersionControlAssetContextLocksMenuHeading", "Unity Version Control Locks"));

	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PlasticReleaseLock", "Release Lock"),
			LOCTEXT("PlasticReleaseLockTooltip", "Release Lock(s) on the selected assets.\nReleasing locks will allow other users to keep working on these files and retrieve locks (on the same branch, in the latest revision)."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Unlocked"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::ExecuteReleaseLocks, SelectedLocks),
				FCanExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::CanReleaseLocks, SelectedLocks)
			)
		);
	}

	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PlasticRemoveLock", "Remove Lock"),
			LOCTEXT("PlasticRemoveLockTooltip", "Remove Lock(s) on the selected assets.\nRemoving locks will allow other users to edit these files anywhere (on any branch) increasing the risk of future merge conflicts."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Unlocked"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::ExecuteRemoveLocks, SelectedLocks),
				FCanExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::CanRemoveLocks, SelectedLocks)
			)
		);
	}

	FString OrganizationName = FPlasticSourceControlModule::Get().GetProvider().GetCloudOrganization();
	if (!OrganizationName.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PlasticLockRulesURL", "Configure Lock Rules"),
			LOCTEXT("PlasticLockRulesURLTooltip", "Navigate to lock rules configuration page in the Unity Dashboard."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Locked"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::VisitLockRulesURLClicked, MoveTemp(OrganizationName)),
				FCanExecuteAction()
			)
		);
	}

	MenuBuilder.EndSection();
}

bool FPlasticSourceControlMenu::CanReleaseLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks) const
{
	for (const FPlasticSourceControlLockRef& Lock : InSelectedLocks)
	{
		// If "Locked" (currently exclusively Checked Out) the lock can be Released, coming back to it's potential underlying "Retained" status if changes where already checked in the branch
		if (Lock->bIsLocked)
		{
			return true;
		}
	}

	return false;
}

bool FPlasticSourceControlMenu::CanRemoveLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks) const
{
	// All "Locked" or "Retained" locks can be Removed
	return (InSelectedLocks.Num() > 0);
}

void FPlasticSourceControlMenu::ExecuteReleaseLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks)
{
	ExecuteUnlock(MoveTemp(InSelectedLocks), false);
}

void FPlasticSourceControlMenu::ExecuteRemoveLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks)
{
	ExecuteUnlock(MoveTemp(InSelectedLocks), true);
}

void FPlasticSourceControlMenu::ExecuteUnlock(TArray<FPlasticSourceControlLockRef>&& InSelectedLocks, const bool bInRemove)
{
	if (!Notification.IsInProgress())
	{
		// Launch a custom "Release/Remove Lock" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		const FString& WorkspaceRoot = Provider.GetPathToWorkspaceRoot();
		const TArray<FString> Files = PlasticSourceControlUtils::LocksToFileNames(WorkspaceRoot, InSelectedLocks);
		TSharedRef<FPlasticUnlock, ESPMode::ThreadSafe> UnlockOperation = ISourceControlOperation::Create<FPlasticUnlock>();
		UnlockOperation->bRemove = bInRemove;
		UnlockOperation->Locks = MoveTemp(InSelectedLocks);
		const ECommandResult::Type Result = Provider.Execute(UnlockOperation, Files, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlMenu::OnSourceControlOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(UnlockOperation->GetInProgressString());
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(UnlockOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

bool FPlasticSourceControlMenu::IsSourceControlConnected() const
{
	const ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
	return Provider.IsEnabled() && Provider.IsAvailable();
}

void FPlasticSourceControlMenu::SyncProjectClicked()
{
	if (!Notification.IsInProgress())
	{
		// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them (reduces friction and solves some user scenario)
		PackageUtils::SaveDirtyPackages();

		// Find and Unlink all loaded packages in Content directory to allow to update them
		PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

		// Launch a custom "SyncAll" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> SyncOperation = ISourceControlOperation::Create<FPlasticSyncAll>();
		const ECommandResult::Type Result = Provider.Execute(SyncOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlMenu::OnSyncAllOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(SyncOperation->GetInProgressString());
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(SyncOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void FPlasticSourceControlMenu::RevertUnchangedClicked()
{
	if (!Notification.IsInProgress())
	{
		// Launch a "RevertUnchanged" Operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticRevertUnchanged, ESPMode::ThreadSafe> RevertUnchangedOperation = ISourceControlOperation::Create<FPlasticRevertUnchanged>();
		const ECommandResult::Type Result = Provider.Execute(RevertUnchangedOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlMenu::OnSourceControlOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation
			Notification.DisplayInProgress(RevertUnchangedOperation->GetInProgressString());
		}
		else
		{
			// Report failure with a notification
			FNotification::DisplayFailure(RevertUnchangedOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void FPlasticSourceControlMenu::RevertAllClicked()
{
	if (!Notification.IsInProgress())
	{
		// Ask the user before reverting all!
		const FText AskRevertAllWarning(LOCTEXT("SourceControlMenu_AskRevertAll", "Revert all modifications into the workspace?\n"
			"This cannot be undone."));
		const EAppReturnType::Type Choice = FMessageDialog::Open(
			EAppMsgCategory::Warning,
			EAppMsgType::OkCancel, AskRevertAllWarning
			, LOCTEXT("SourceControlMenu_AskRevertAllTitle", "Revert All?")
		);
		if (Choice == EAppReturnType::Ok)
		{
			// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them (reduces friction and solves some user scenario)
			PackageUtils::SaveDirtyPackages();

			// Find and Unlink all packages in Content directory to allow to update them
			PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

			// Launch a "RevertAll" Operation
			FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
			TSharedRef<FPlasticRevertAll, ESPMode::ThreadSafe> RevertAllOperation = ISourceControlOperation::Create<FPlasticRevertAll>();
			const ECommandResult::Type Result = Provider.Execute(RevertAllOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlMenu::OnRevertAllOperationComplete));
			if (Result == ECommandResult::Succeeded)
			{
				// Display an ongoing notification during the whole operation
				Notification.DisplayInProgress(RevertAllOperation->GetInProgressString());
			}
			else
			{
				// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
				FNotification::DisplayFailure(RevertAllOperation.Get());
			}
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void FPlasticSourceControlMenu::SwitchToPartialWorkspaceClicked()
{
	if (!Notification.IsInProgress())
	{
		// Ask the user before switching to Partial Workspace. It's not possible to switch back with local changes!
		const FText SwitchToPartialQuestion(LOCTEXT("SourceControlMenu_AskSwitchToPartialWorkspace", "Switch to Gluon partial workspace?\n"
			"Please note that in order to switch back to a regular workspace you will need to undo any local changes."));
		const EAppReturnType::Type Choice = FMessageDialog::Open(
			EAppMsgCategory::Info,
			EAppMsgType::OkCancel, SwitchToPartialQuestion
			, LOCTEXT("SourceControlMenu_SwitchToPartialTitle", "Switch to Gluon partial workspace?")
		);
		if (Choice == EAppReturnType::Ok)
		{
			// Launch a "SwitchToPartialWorkspace" Operation
			FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
			TSharedRef<FPlasticSwitchToPartialWorkspace, ESPMode::ThreadSafe> SwitchOperation = ISourceControlOperation::Create<FPlasticSwitchToPartialWorkspace>();
			const ECommandResult::Type Result = Provider.Execute(SwitchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlMenu::OnSourceControlOperationComplete));
			if (Result == ECommandResult::Succeeded)
			{
				// Display an ongoing notification during the whole operation
				Notification.DisplayInProgress(SwitchOperation->GetInProgressString());
			}
			else
			{
				// Report failure with a notification
				FNotification::DisplayFailure(SwitchOperation.Get());
			}
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

bool FPlasticSourceControlMenu::CanSwitchToPartialWorkspace() const
{
	return !FPlasticSourceControlModule::Get().GetProvider().IsPartialWorkspace();
}

void FPlasticSourceControlMenu::ShowSourceControlEditorPreferences() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->ShowViewer("Editor", "General", "LoadingSaving");
	}
}

void FPlasticSourceControlMenu::ShowSourceControlProjectSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->ShowViewer("Project", "Editor", "SourceControlPreferences");
	}
}

void FPlasticSourceControlMenu::ShowSourceControlPlasticScmProjectSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->ShowViewer("Project", "Editor", "PlasticSourceControlProjectSettings");
	}
}

void FPlasticSourceControlMenu::VisitDocsURLClicked() const
{
	// Grab the URL from the uplugin file
	const TSharedPtr<IPlugin> Plugin = FPlasticSourceControlModule::GetPlugin();
	if (Plugin.IsValid())
	{
		FPlatformProcess::LaunchURL(*Plugin->GetDescriptor().DocsURL, NULL, NULL);
	}
}

void FPlasticSourceControlMenu::VisitSupportURLClicked() const
{
	// Grab the URL from the uplugin file
	const TSharedPtr<IPlugin> Plugin = FPlasticSourceControlModule::GetPlugin();
	if (Plugin.IsValid())
	{
		FPlatformProcess::LaunchURL(*Plugin->GetDescriptor().SupportURL, NULL, NULL);
	}
}

void FPlasticSourceControlMenu::VisitLockRulesURLClicked(const FString InOrganizationName) const
{
	PlasticSourceControlUtils::OpenLockRulesInCloudDashboard(InOrganizationName);
}

void FPlasticSourceControlMenu::OpenDesktopApplication() const
{
	PlasticSourceControlUtils::OpenDesktopApplication();
}

void FPlasticSourceControlMenu::OpenBranchesWindow() const
{
	FPlasticSourceControlModule::Get().GetBranchesWindow().OpenTab();
}

void FPlasticSourceControlMenu::OpenChangesetsWindow() const
{
	FPlasticSourceControlModule::Get().GetChangesetsWindow().OpenTab();
}

void FPlasticSourceControlMenu::OpenLocksWindow() const
{
	FPlasticSourceControlModule::Get().GetLocksWindow().OpenTab();
}

void FPlasticSourceControlMenu::OnSyncAllOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	OnSourceControlOperationComplete(InOperation, InResult);

	// Reload packages that where updated by the Sync operation (and the current map if needed)
	TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticSyncAll>(InOperation);
	PackageUtils::ReloadPackages(Operation->UpdatedFiles);
}

void FPlasticSourceControlMenu::OnRevertAllOperationComplete(const FSourceControlOperationRef & InOperation, ECommandResult::Type InResult)
{
	OnSourceControlOperationComplete(InOperation, InResult);

	// Reload packages that where updated by the Revert operation (and the current map if needed)
	TSharedRef<FPlasticRevertAll, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticRevertAll>(InOperation);
	PackageUtils::ReloadPackages(Operation->UpdatedFiles);
}

void FPlasticSourceControlMenu::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

// TODO rework the menus with sub-menus like in the POC branch
void FPlasticSourceControlMenu::AddMenuExtension(FToolMenuSection& Menu)
{
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();

	Menu.AddMenuEntry(
		"PlasticSync",
		LOCTEXT("PlasticSync",			"Sync/Update Workspace"),
		LOCTEXT("PlasticSyncTooltip",	"Update the workspace to the latest changeset of the branch, and reload all affected assets."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Sync"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::SyncProjectClicked),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"PlasticRevertUnchanged",
		LOCTEXT("PlasticRevertUnchanged",			"Revert Unchanged"),
		LOCTEXT("PlasticRevertUnchangedTooltip",	"Revert checked-out but unchanged files in the workspace."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Revert"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::RevertUnchangedClicked),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"PlasticRevertAll",
		LOCTEXT("PlasticRevertAll",			"Revert All"),
		LOCTEXT("PlasticRevertAllTooltip",	"Revert all files in the workspace to their controlled/unchanged state."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Revert"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::RevertAllClicked),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"SwitchToPartialWorkspace",
		LOCTEXT("SwitchToPartialWorkspace",			"Switch to Gluon Partial Workspace"),
		LOCTEXT("SwitchToPartialWorkspaceTooltip",	"Update the workspace to a Gluon partial mode for a simplified workflow.\n"
			"Allows to update and check in files individually as opposed to the whole workspace.\nIt doesn't work with branches or shelves."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Cut"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::SwitchToPartialWorkspaceClicked),
			FCanExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::CanSwitchToPartialWorkspace)
		)
	);

	Menu.AddMenuEntry(
		"SourceControlEditorPreferences",
		LOCTEXT("SourceControlEditorPreferences", "Editor Preferences - Source Control"),
		LOCTEXT("SourceControlEditorPreferencesTooltip", "Open the Load & Save section with Source Control in the Editor Preferences."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorPreferences.TabIcon"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::ShowSourceControlEditorPreferences),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"SourceControlProjectSettings",
		LOCTEXT("SourceControlProjectSettings",			"Project Settings - Revision Control"),
		LOCTEXT("SourceControlProjectSettingsTooltip",	"Open the Revision Control section in the Project Settings."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::ShowSourceControlProjectSettings),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"PlasticProjectSettings",
		LOCTEXT("PlasticProjectSettings",			"Project Settings - Source Control - Unity Version Control"),
		LOCTEXT("PlasticProjectSettingsTooltip",	"Open the Unity Version Control (formerly Plastic SCM) section in the Project Settings."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ProjectSettings.TabIcon"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::ShowSourceControlPlasticScmProjectSettings),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"PlasticDocsURL",
		LOCTEXT("PlasticDocsURL",			"Plugin's Documentation"),
		LOCTEXT("PlasticDocsURLTooltip",	"Visit documentation of the plugin on Github."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Documentation"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::VisitDocsURLClicked),
			FCanExecuteAction()
		)
	);

	Menu.AddMenuEntry(
		"PlasticSupportURL",
		LOCTEXT("PlasticSupportURL",		"Unity Version Control Support"),
		LOCTEXT("PlasticSupportURLTooltip",	"Submit a support request for Unity Version Control (formerly Plastic SCM)."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Support"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::VisitSupportURLClicked),
			FCanExecuteAction()
		)
	);

	FString OrganizationName = FPlasticSourceControlModule::Get().GetProvider().GetCloudOrganization();
	if (!OrganizationName.IsEmpty())
	{
		Menu.AddMenuEntry(
			"PlasticLockRulesURL",
			LOCTEXT("PlasticLockRulesURL", "Configure Lock Rules"),
			LOCTEXT("PlasticLockRulesURLTooltip", "Navigate to lock rules configuration page in the Unity Dashboard."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Locked"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::VisitLockRulesURLClicked, MoveTemp(OrganizationName)),
				FCanExecuteAction()
			)
		);
	}

	Menu.AddMenuEntry(
		"PlasticDesktopApp",
		TAttribute<FText>::CreateLambda([&Provider]()
			{
				return Provider.IsPartialWorkspace() ? LOCTEXT("PlasticGluon", "Open in Gluon") : LOCTEXT("PlasticDesktopApp", "Open in Desktop App");
			}),
		TAttribute<FText>::CreateLambda([&Provider]()
			{
				return Provider.IsPartialWorkspace() ? LOCTEXT("PlasticDesktopAppTooltip", "Open the workspace in Unity Version Control Gluon Application.") : LOCTEXT("PlasticGluonTooltip", "Open the workspace in Unity Version Control Desktop Application.");
			}),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"),
		FUIAction(FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::OpenDesktopApplication))
	);

	AddViewBranches(Menu);
	AddViewChangesets(Menu);
	AddViewLocks(Menu);
}

void FPlasticSourceControlMenu::AddViewBranches(FToolMenuSection& Menu)
{
	Menu.AddMenuEntry(
		TEXT("PlasticBranchesWindow"),
		LOCTEXT("PlasticBranchesWindow", "View Branches"),
		LOCTEXT("PlasticBranchesWindowTooltip", "Open the Branches window."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Branch"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::OpenBranchesWindow),
			FCanExecuteAction()
		)
	);
}

void FPlasticSourceControlMenu::AddViewChangesets(FToolMenuSection& Menu)
{
	Menu.AddMenuEntry(
		TEXT("PlasticChangesetsWindow"),
		LOCTEXT("PlasticChangesetsWindow", "View Changesets"),
		LOCTEXT("PlasticChangesetsWindowTooltip", "Open the Changesets window."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.History"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::OpenChangesetsWindow),
			FCanExecuteAction()
		)
	);
}

void FPlasticSourceControlMenu::AddViewLocks(FToolMenuSection& Menu)
{
	const bool bVersionSupportsSmartLocks = FPlasticSourceControlModule::Get().GetProvider().GetPlasticScmVersion() >= PlasticSourceControlVersions::SmartLocks;

	Menu.AddMenuEntry(
		TEXT("PlasticLocksWindow"),
		LOCTEXT("PlasticLocksWindow", "View Locks"),
		LOCTEXT("PlasticLocksWindowTooltip", "Open the Locks window."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Locked"),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FPlasticSourceControlMenu::OpenLocksWindow),
			FCanExecuteAction::CreateLambda([bVersionSupportsSmartLocks]() { return bVersionSupportsSmartLocks; })
		)
	);
}

#undef LOCTEXT_NAMESPACE
