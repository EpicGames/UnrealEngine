// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#include "Notification.h"
#include "AssetRegistry/AssetData.h"

class FMenuBuilder;
struct FToolMenuSection;

typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;

/** Unity Version Control extension of the Source Control toolbar menu */
class FPlasticSourceControlMenu
{
public:
	void Register();
	void Unregister();

	/** This functions will be bound to appropriate Command. */
	void SyncProjectClicked();
	void RevertUnchangedClicked();
	void RevertAllClicked();
	void SwitchToPartialWorkspaceClicked();
	bool CanSwitchToPartialWorkspace() const;
	void ShowSourceControlEditorPreferences() const;
	void ShowSourceControlProjectSettings() const;
	void ShowSourceControlPlasticScmProjectSettings() const;
	void VisitDocsURLClicked() const;
	void VisitSupportURLClicked() const;
	void VisitLockRulesURLClicked(const FString InOrganizationName) const;
	void OpenDesktopApplication() const;
	void OpenBranchesWindow() const;
	void OpenChangesetsWindow() const;
	void OpenLocksWindow() const;

private:
	bool IsSourceControlConnected() const;

	void AddMenuExtension(FToolMenuSection& Menu);
	void AddViewBranches(FToolMenuSection& Menu);
	void AddViewChangesets(FToolMenuSection& Menu);
	void AddViewLocks(FToolMenuSection& Menu);

	/** Extends the UE5 toolbar with a status bar widget to display the current branch and open the branch tab */
	void ExtendToolbarWithStatusBarWidget();

	/** Extends the main Revision Control menu from the toolbar at the bottom-right. */
	void ExtendRevisionControlMenu();
	/** Extends the content browser asset context menu with Admin revision control options. */
	void ExtendAssetContextMenu();
	/** Called to generate concert asset context menu. */
	void GeneratePlasticAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> InAssetObjectPaths);

	bool CanRemoveLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks) const;
	bool CanReleaseLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks) const;
	void ExecuteRemoveLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks);
	void ExecuteReleaseLocks(TArray<FPlasticSourceControlLockRef> InSelectedLocks);
	void ExecuteUnlock(TArray<FPlasticSourceControlLockRef>&& InSelectedLocks, const bool bInRemove);

private:
	/** Tracks if the menu extension has been registered with the editor or not */
	bool bHasRegistered = false;

	/** Ongoing notification for a long-running asynchronous source control operation, if any */
	FNotification Notification;

	/** Name of the menu extension going into the global Revision Control (on the toolbar at the bottom-right) */
	static FName UnityVersionControlMainMenuOwnerName;
	/** Name of the asset context menu extension for admin actions over Locks */
	static FName UnityVersionControlAssetContextLocksMenuOwnerName;
	/** Name of status bar extension to display the current branch  */
	static FName UnityVersionControlStatusBarMenuOwnerName;

	/** Delegates called when a source control operation has completed */
	void OnSyncAllOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnRevertAllOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	/** Generic delegate and notification handler */
	void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
};
