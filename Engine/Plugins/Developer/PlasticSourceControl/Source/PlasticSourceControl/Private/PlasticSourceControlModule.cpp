// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlModule.h"

#include "IPlasticSourceControlWorker.h"

#include "Interfaces/IPluginManager.h"
#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

void FPlasticSourceControlModule::StartupModule()
{
	// Register our operations (implemented in PlasticSourceControlOperations.cpp by sub-classing from Engine\Source\Developer\SourceControl\Public\SourceControlOperations.h)
	IPlasticSourceControlWorker::RegisterWorkers(PlasticSourceControlProvider);

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature("SourceControl", &PlasticSourceControlProvider);

	/// Register our tab windows here as they needs to be ready for the editor to reload at startup
	PlasticSourceControlBranchesWindow.Register();
	PlasticSourceControlChangesetsWindow.Register();
	PlasticSourceControlLocksWindow.Register();
}

void FPlasticSourceControlModule::ShutdownModule()
{
	// shut down the provider, as this module is going away
	PlasticSourceControlProvider.Close();

	PlasticSourceControlBranchesWindow.Unregister();
	PlasticSourceControlChangesetsWindow.Unregister();
	PlasticSourceControlLocksWindow.Unregister();

	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature("SourceControl", &PlasticSourceControlProvider);
}

FPlasticSourceControlModule& FPlasticSourceControlModule::Get()
{
	return FModuleManager::GetModuleChecked<FPlasticSourceControlModule>("PlasticSourceControl");
}

bool FPlasticSourceControlModule::IsLoaded()
{
	return FModuleManager::Get().IsModuleLoaded("PlasticSourceControl");
}

const TSharedPtr<IPlugin> FPlasticSourceControlModule::GetPlugin()
{
	return IPluginManager::Get().FindPlugin(TEXT("PlasticSourceControl"));;
}

IMPLEMENT_MODULE(FPlasticSourceControlModule, PlasticSourceControl);

#undef LOCTEXT_NAMESPACE
