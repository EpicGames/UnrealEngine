// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlWorkspaceCreation.h"

#include "PlasticSourceControlBranchesWindow.h"
#include "PlasticSourceControlChangesetsWindow.h"
#include "PlasticSourceControlLocksWindow.h"

/**
 * PlasticSourceControl is the official Unity Version Control Plugin for Unreal Engine
 *
 * Written and contributed by Sebastien Rombauts (sebastien.rombauts@gmail.com) for Codice Software
 */
class FPlasticSourceControlModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Access the Plastic source control provider */
	FPlasticSourceControlProvider& GetProvider()
	{
		return PlasticSourceControlProvider;
	}
	const FPlasticSourceControlProvider& GetProvider() const
	{
		return PlasticSourceControlProvider;
	}

	/** Access the controller to create a new workspace */
	FPlasticSourceControlWorkspaceCreation& GetWorkspaceCreation()
	{
		return PlasticSourceControlWorkspaceCreation;
	}

	FPlasticSourceControlBranchesWindow& GetBranchesWindow()
	{
		return PlasticSourceControlBranchesWindow;
	}

	FPlasticSourceControlChangesetsWindow& GetChangesetsWindow()
	{
		return PlasticSourceControlChangesetsWindow;
	}

	FPlasticSourceControlLocksWindow& GetLocksWindow()
	{
		return PlasticSourceControlLocksWindow;
	}

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, asserts if the module is not loaded yet or unloaded already.
	 */
	static FPlasticSourceControlModule& Get();

	/**
	 * Checks whether the module is currently loaded.
	 */
	static bool IsLoaded();

	/**
	 * Finds information of the plugin.
	 *
	 * @return	 Pointer to the plugin's information, or nullptr.
	 */
	static const TSharedPtr<class IPlugin> GetPlugin();

private:
	/** The Plastic source control provider */
	FPlasticSourceControlProvider PlasticSourceControlProvider;

	/** Dockable windows adding advanced features to the plugin */
	FPlasticSourceControlBranchesWindow PlasticSourceControlBranchesWindow;
	FPlasticSourceControlChangesetsWindow PlasticSourceControlChangesetsWindow;
	FPlasticSourceControlLocksWindow PlasticSourceControlLocksWindow;

	/** Logic to create a new workspace */
	FPlasticSourceControlWorkspaceCreation PlasticSourceControlWorkspaceCreation;
};
