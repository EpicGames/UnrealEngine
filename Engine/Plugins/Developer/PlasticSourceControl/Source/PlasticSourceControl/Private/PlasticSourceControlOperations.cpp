// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlOperations.h"

#include "PackageUtils.h"
#include "PlasticSourceControlBranch.h"
#include "PlasticSourceControlChangeset.h"
#include "PlasticSourceControlCommand.h"
#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlParsers.h"
#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlSettings.h"
#include "PlasticSourceControlShell.h"
#include "PlasticSourceControlState.h"
#include "PlasticSourceControlUtils.h"
#include "PlasticSourceControlVersions.h"
#include "ScopedTempFile.h"

#include "Algo/AnyOf.h"
#include "Algo/NoneOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "Logging/MessageLog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include "ISourceControlModule.h"
#include "SourceControlOperations.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

template<typename Type>
static FPlasticSourceControlWorkerRef InstantiateWorker(FPlasticSourceControlProvider& PlasticSourceControlProvider)
{
	return MakeShareable(new Type(PlasticSourceControlProvider));
}

void IPlasticSourceControlWorker::RegisterWorkers(FPlasticSourceControlProvider& PlasticSourceControlProvider)
{
	// Register our operations (implemented in PlasticSourceControlOperations.cpp by sub-classing from Engine\Source\Developer\SourceControl\Public\SourceControlOperations.h)
	PlasticSourceControlProvider.RegisterWorker("Connect", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticConnectWorker>));
	PlasticSourceControlProvider.RegisterWorker("CheckOut", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticCheckOutWorker>));
	PlasticSourceControlProvider.RegisterWorker("UpdateStatus", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticUpdateStatusWorker>));
	PlasticSourceControlProvider.RegisterWorker("MarkForAdd", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticMarkForAddWorker>));
	PlasticSourceControlProvider.RegisterWorker("Delete", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticDeleteWorker>));
	PlasticSourceControlProvider.RegisterWorker("Revert", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticRevertWorker>));
	PlasticSourceControlProvider.RegisterWorker("RevertUnchanged", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticRevertUnchangedWorker>));
	PlasticSourceControlProvider.RegisterWorker("RevertAll", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticRevertAllWorker>));
	PlasticSourceControlProvider.RegisterWorker("RevertToRevision", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticRevertToRevisionWorker>));
	PlasticSourceControlProvider.RegisterWorker("SwitchToPartialWorkspace", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticSwitchToPartialWorkspaceWorker>));
	PlasticSourceControlProvider.RegisterWorker("GetLocks", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetLocksWorker>));
	PlasticSourceControlProvider.RegisterWorker("Unlock", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticUnlockWorker>));
	PlasticSourceControlProvider.RegisterWorker("GetBranches", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetBranchesWorker>));
	PlasticSourceControlProvider.RegisterWorker("Switch", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticSwitchWorker>));
	PlasticSourceControlProvider.RegisterWorker("Merge", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticMergeBranchWorker>));
	PlasticSourceControlProvider.RegisterWorker("CreateBranch", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticCreateBranchWorker>));
	PlasticSourceControlProvider.RegisterWorker("RenameBranch", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticRenameBranchWorker>));
	PlasticSourceControlProvider.RegisterWorker("DeleteBranches", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticDeleteBranchesWorker>));
	PlasticSourceControlProvider.RegisterWorker("GetChangesets", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetChangesetsWorker>));
	PlasticSourceControlProvider.RegisterWorker("GetChangesetFiles", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetChangesetFilesWorker>));
	PlasticSourceControlProvider.RegisterWorker("MakeWorkspace", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticMakeWorkspaceWorker>));
	PlasticSourceControlProvider.RegisterWorker("Sync", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticSyncWorker>));
	PlasticSourceControlProvider.RegisterWorker("SyncAll", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticSyncWorker>));
	PlasticSourceControlProvider.RegisterWorker("CheckIn", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticCheckInWorker>));
	PlasticSourceControlProvider.RegisterWorker("Copy", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticCopyWorker>));
	PlasticSourceControlProvider.RegisterWorker("Resolve", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticResolveWorker>));

	PlasticSourceControlProvider.RegisterWorker("UpdateChangelistsStatus", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetPendingChangelistsWorker>));
	PlasticSourceControlProvider.RegisterWorker("NewChangelist", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticNewChangelistWorker>));
	PlasticSourceControlProvider.RegisterWorker("DeleteChangelist", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticDeleteChangelistWorker>));
	PlasticSourceControlProvider.RegisterWorker("EditChangelist", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticEditChangelistWorker>));
	PlasticSourceControlProvider.RegisterWorker("MoveToChangelist", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticReopenWorker>));

	PlasticSourceControlProvider.RegisterWorker("Shelve", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticShelveWorker>));
	PlasticSourceControlProvider.RegisterWorker("Unshelve", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticUnshelveWorker>));
	PlasticSourceControlProvider.RegisterWorker("DeleteShelved", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticDeleteShelveWorker>));

	PlasticSourceControlProvider.RegisterWorker("GetChangelistDetails", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetChangelistDetailsWorker>));
	PlasticSourceControlProvider.RegisterWorker("GetFile", FGetPlasticSourceControlWorker::CreateStatic(&InstantiateWorker<FPlasticGetFileWorker>));
}


FName FPlasticRevertUnchanged::GetName() const
{
	return "RevertUnchanged";
}

FText FPlasticRevertUnchanged::GetInProgressString() const
{
	return LOCTEXT("SourceControl_RevertUnchanged", "Reverting unchanged file(s) in Revision Control...");
}

FName FPlasticSyncAll::GetName() const
{
	return "SyncAll";
}

FText FPlasticSyncAll::GetInProgressString() const
{
	if (!Revision.IsEmpty())
	{
		return FText::Format(LOCTEXT("SourceControl_SyncAll", "Switching the workspace to changeset {0}..."), FText::FromString(Revision));
	}

	return FSync::GetInProgressString();
}

FName FPlasticRevertAll::GetName() const
{
	return "RevertAll";
}

FText FPlasticRevertAll::GetInProgressString() const
{
	return LOCTEXT("SourceControl_RevertAll", "Reverting checked-out file(s) in Revision Control...");
}

FName FPlasticRevertToRevision::GetName() const
{
	return "RevertToRevision";
}

FText FPlasticRevertToRevision::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_RevertToRevision", "Reverting selected file(s) to revision {0}..."), FText::AsNumber(ChangesetId));
}

FName FPlasticMakeWorkspace::GetName() const
{
	return "MakeWorkspace";
}

FText FPlasticMakeWorkspace::GetInProgressString() const
{
	return LOCTEXT("SourceControl_MakeWorkspace", "Creating a new Repository and Workspace...");
}

FName FPlasticSwitchToPartialWorkspace::GetName() const
{
	return "SwitchToPartialWorkspace";
}

FText FPlasticSwitchToPartialWorkspace::GetInProgressString() const
{
	return LOCTEXT("SourceControl_SwitchToPartialWorkspace", "Switching to a Partial/Gluon Workspace...");
}

FName FPlasticGetLocks::GetName() const
{
	return "GetLocks";
}

FText FPlasticGetLocks::GetInProgressString() const
{
	return LOCTEXT("SourceControl_GetLocks", "Getting the list of locks...");
}

FName FPlasticUnlock::GetName() const
{
	return "Unlock";
}

FText FPlasticUnlock::GetInProgressString() const
{
	if (bRemove)
		return LOCTEXT("SourceControl_Unlock_Remove", "Removing Lock(s)...");
	else
		return LOCTEXT("SourceControl_Unlock_Release", "Releasing Lock(s)...");
}

FName FPlasticGetBranches::GetName() const
{
	return "GetBranches";
}

FText FPlasticGetBranches::GetInProgressString() const
{
	return LOCTEXT("SourceControl_GetBranches", "Getting the list of branches...");
}

FName FPlasticSwitch::GetName() const
{
	return "Switch";
}

FText FPlasticSwitch::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_SwitchToBranch", "Switching the workspace to branch {0}..."), FText::FromString(BranchName));
}

FName FPlasticMergeBranch::GetName() const
{
	return "Merge";
}

FText FPlasticMergeBranch::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_MergeBranch", "Merging branch {0} to the current one..."), FText::FromString(BranchName));
}

FName FPlasticCreateBranch::GetName() const
{
	return "CreateBranch";
}

FText FPlasticCreateBranch::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_CreateBranch", "Creating new child branch {0}..."), FText::FromString(BranchName));
}


FName FPlasticRenameBranch::GetName() const
{
	return "RenameBranch";
}

FText FPlasticRenameBranch::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_RenameBranch", "Renaming the branch {0} to {1}..."), FText::FromString(OldName), FText::FromString(NewName));
}

FName FPlasticDeleteBranches::GetName() const
{
	return "DeleteBranches";
}

FText FPlasticDeleteBranches::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_DeleteBranches", "Deleting branch(es) {0}..."), FText::FromString(BranchNames[0]));
}

FName FPlasticGetChangesets::GetName() const
{
	return "GetChangesets";
}

FText FPlasticGetChangesets::GetInProgressString() const
{
	return LOCTEXT("SourceControl_GetChangesets", "Getting the list of changesets...");
}

FName FPlasticGetChangesetFiles::GetName() const
{
	return "GetChangesetFiles";
}

FText FPlasticGetChangesetFiles::GetInProgressString() const
{
	return FText::Format(LOCTEXT("SourceControl_GetChangesetFiles", "Getting the list of files in changeset {0}..."), FText::AsNumber(Changeset->ChangesetId));
}


static bool AreAllFiles(const TArray<FString>& InFiles)
{
	for (const FString& File : InFiles)
	{
		if (File.IsEmpty() || File[File.Len() - 1] == TEXT('/'))
			return false;
	}
	return true;
}


FName FPlasticConnectWorker::GetName() const
{
	return "Connect";
}

bool FPlasticConnectWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticConnectWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FConnect, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FConnect>(InCommand.Operation);

	if (GetProvider().IsPlasticAvailable())
	{
		// Get workspace name
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::GetWorkspaceName(InCommand.PathToWorkspaceRoot, InCommand.WorkspaceName, InCommand.ErrorMessages);
		if (InCommand.bCommandSuccessful)
		{
			// Get current branch, repository, server URL and check the connection
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCheckConnection(InCommand.WorkspaceSelector, InCommand.BranchName, InCommand.RepositoryName, InCommand.ServerUrl, InCommand.InfoMessages, InCommand.ErrorMessages);
			if (InCommand.bCommandSuccessful)
			{
				InCommand.InfoMessages.Add(TEXT("Connected successfully"));

				// Now update the status of assets in the Content directory
				// but only on real (re-)connection (but not each time Login() is called by Rename or Fixup Redirector command to check connection)
				// and only if enabled in the settings
				if (!PlasticSourceControlShell::GetShellIsWarmedUp() && GetProvider().AccessSettings().GetUpdateStatusAtStartup())
				{
					PlasticSourceControlShell::SetShellIsWarmedUp();
					TArray<FString> ContentDir;
					ContentDir.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
					PlasticSourceControlUtils::RunUpdateStatus(ContentDir, PlasticSourceControlUtils::EStatusSearchType::All, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
				}
			}
			else
			{
				if (InCommand.ErrorMessages.Num() > 0)
				{
					Operation->SetErrorText(FText::FromString(InCommand.ErrorMessages[0]));
				}
				else
				{
					const FText ErrorText(LOCTEXT("FailedToConnect", "Failed to connect to the Unity Version Control (formerly Plastic SCM) server."));
					Operation->SetErrorText(ErrorText);
					InCommand.ErrorMessages.Add(ErrorText.ToString());
				}
			}
		}
		else
		{
			const FText ErrorText(LOCTEXT("NotAPlasticRepository", "Failed to enable Unity Version Control (formerly Plastic SCM). You need to create a workspace for the project first."));
			Operation->SetErrorText(ErrorText);
			InCommand.ErrorMessages.Add(ErrorText.ToString());
		}
	}
	else
	{
		const FText ErrorText(LOCTEXT("PlasticScmCliUnavaillable", "Failed to launch Unity Version Control (formerly Plastic SCM) 'cm' command line tool. You need to install it and make sure it is correctly configured with your credentials."));
		Operation->SetErrorText(ErrorText);
		InCommand.ErrorMessages.Add(ErrorText.ToString());
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticConnectWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticConnectWorker::UpdateStates);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}


static void UpdateChangelistState(FPlasticSourceControlProvider& SCCProvider, const FPlasticSourceControlChangelist& InChangelist, const TArray<FPlasticSourceControlState>& InStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlastic::UpdateChangelistState);

	if (InChangelist.IsInitialized())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = SCCProvider.GetStateInternal(InChangelist);

		for (const FPlasticSourceControlState& InState : InStates)
		{
			if (!InState.IsPendingChanges())
			{
				continue;
			}

			// Add a shared reference to the state of the file, that will then be updated by PlasticSourceControlUtils::UpdateCachedStates()
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = SCCProvider.GetStateInternal(InState.GetFilename());
			ChangelistState->Files.AddUnique(State);

			// Keep the changelist stored with cached file state in sync with the actual changelist that owns this file.
			State->Changelist = InChangelist;
		}
	}
}


FName FPlasticCheckOutWorker::GetName() const
{
	return "CheckOut";
}

bool FPlasticCheckOutWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCheckOutWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	if (!GetProvider().IsPartialWorkspace())
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("checkout"), TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
	}
	else
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial checkout"), TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// now update the status of our files
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticCheckOutWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCheckOutWorker::UpdateStates);

	// If files have been checked-out directly to a CL, modify the cached state to reflect it (defaults to the Default changelist).
	UpdateChangelistState(GetProvider(), InChangelist, States);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

bool DeleteChangelist(const FPlasticSourceControlProvider& PlasticSourceControlProvider, const FPlasticSourceControlChangelist& InChangelist, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Parameters;
	TArray<FString> Files;
	if (PlasticSourceControlProvider.GetPlasticScmVersion() < PlasticSourceControlVersions::NewChangelistFileArgs)
	{
		Parameters.Add(TEXT("rm"));
		Files.Add(InChangelist.GetName());
		return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, Files, OutResults, OutErrorMessages);
	}
	else
	{
		Parameters.Add(TEXT("delete"));
		const FScopedTempFile ChangelistNameFile(InChangelist.GetName());
		Parameters.Add(FString::Printf(TEXT("--namefile=\"%s\""), *ChangelistNameFile.GetFilename()));
		UE_LOGF(LogSourceControl, Verbose, "DeleteChangelist(%ls)", *InChangelist.GetName());
		return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, Files, OutResults, OutErrorMessages);
	}
}

static TArray<FString> FileNamesFromFileStates(const TArray<FSourceControlStateRef>& InFileStates)
{
	TArray<FString> Files;
	Files.Reserve(InFileStates.Num());
	for (const FSourceControlStateRef& FileState : InFileStates)
	{
		Files.Add(FileState->GetFilename());
	}
	return Files;
}

static TArray<FString> GetFilesFromCommand(FPlasticSourceControlProvider& PlasticSourceControlProvider, FPlasticSourceControlCommand& InCommand)
{
	TArray<FString> Files;

	if (InCommand.Changelist.IsInitialized() && InCommand.Files.IsEmpty())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = PlasticSourceControlProvider.GetStateInternal(InCommand.Changelist);
		Files = FileNamesFromFileStates(ChangelistState->Files);
	}
	else
	{
		Files = InCommand.Files;
	}
	return Files;
}


FName FPlasticCheckInWorker::GetName() const
{
	return "CheckIn";
}

bool FPlasticCheckInWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCheckInWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCheckIn>(InCommand.Operation);
	FText Description = Operation->GetDescription();

	const TArray<FString> Files = GetFilesFromCommand(GetProvider(), InCommand);

	if (Files.Num() == 0)
	{
		UE_LOGF(LogSourceControl, Warning, "Checkin: No files provided");
		return false;
	}

	// Add Private files to source control to be able to check them in
	TArray<FString> FilesToAdd;
	for (const FString& File : Files)
	{
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(File);
		if (!State->IsSourceControlled())
		{
			FilesToAdd.Add(File);
		}
	}
	if (FilesToAdd.Num() > 0)
	{
		UE_LOGF(LogSourceControl, Log, "CheckIn: Adding %d file(s) to source control", FilesToAdd.Num());
		if (!GetProvider().IsPartialWorkspace())
		{
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("add"), TArray<FString>(), FilesToAdd, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		else
		{
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial add"), TArray<FString>(), FilesToAdd, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
	}

	if (InCommand.Changelist.IsInitialized())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);

		if (Description.IsEmpty())
		{
			Description = ChangelistState->GetDescriptionText();
		}

		InChangelist = InCommand.Changelist;
	}

	UE_LOGF(LogSourceControl, Verbose, "CheckIn: %d file(s) Description: '%ls'", Files.Num(), *Description.ToString());

	// make a temp file to place our commit message in
	const FScopedTempFile CommitMsgFile(Description);
	if (!CommitMsgFile.GetFilename().IsEmpty())
	{
		TArray<FString> Parameters;
		Parameters.Add(FString::Printf(TEXT("--commentsfile=\"%s\""), *CommitMsgFile.GetFilename()));
		Parameters.Add(TEXT("--all"));			// Also files Changed (not CheckedOut) and Moved/Deleted Locally
		if (!GetProvider().IsPartialWorkspace())
		{
			//  NOTE: --update added as #23 but removed as #32 because most assets are locked by the Unreal Editor
		//  Parameters.Add(TEXT("--update")); // Processes the update-merge automatically if it eventually happens.
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("checkin"), Parameters, Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		else
		{
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial checkin"), Parameters, Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		if (InCommand.bCommandSuccessful)
		{
			Operation->SetSuccessMessage(PlasticSourceControlParsers::ParseCheckInResults(InCommand.InfoMessages));
			UE_LOGF(LogSourceControl, Log, "CheckIn successful");
		}

		if (InChangelist.IsInitialized() && !InChangelist.IsDefault())
		{
			// NOTE: we need to explicitly delete persistent changelists when we submit its content, except for the Default changelist
			DeleteChangelist(GetProvider(), InChangelist, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
	}

	// now update the status of our files
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticCheckInWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCheckInWorker::UpdateStates);

	// Is the submit of a full changelist (from the View Changelists window) or a set of files (from the Submit Content window)?
	if (InChangelist.IsInitialized())
	{
		if (InChangelist.IsDefault())
		{
			// Remove all the files from the default changelist state, since they have been submitted, but we didn't delete the changelist itself
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> DefaultChangelist = GetProvider().GetStateInternal(FPlasticSourceControlChangelist::DefaultChangelist);
			DefaultChangelist->Files.Empty();
		}
		else
		{
			// Remove references to the changelist (else it later creates a phantom of the old changelist in the cache)
			for (const FPlasticSourceControlState& NewState : States)
			{
				TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
				State->Changelist.Reset();
			}

			GetProvider().RemoveChangelistFromCache(InChangelist);
		}
	}
	else
	{
		// Update affected changelists if any
		for (const FPlasticSourceControlState& NewState : States)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
			if (State->Changelist.IsInitialized())
			{
				// 1- Remove these files from their previous changelist
				TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(State->Changelist);
				PreviousChangelist->Files.Remove(State);
				// 2- And reset the reference to their previous changelist
				State->Changelist.Reset();
			}
		}
	}

	// Remove any deleted files from status cache
	for (const auto& State : States)
	{
		// Note: a file that was deleted and submitted now appears as "Private", that is, not in Source Control anymore
		if (!State.IsSourceControlled())
		{
			GetProvider().RemoveFileFromCache(State.GetFilename());
		}
	}

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticMarkForAddWorker::GetName() const
{
	return "MarkForAdd";
}

bool FPlasticMarkForAddWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticMarkForAddWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	if (InCommand.Files.Num() > 0)
	{
		TArray<FString> Parameters;
		Parameters.Add(TEXT("--parents")); // NOTE: deprecated in 8.0.16.3100 when it became the default https://www.plasticscm.com/download/releasenotes/8.0.16.3100
		// Note: using "?" is a workaround to trigger the Plastic's "SkipIgnored" internal flag meaning "don't add file that are ignored":
		//          options.SkipIgnored = cla.GetWildCardArguments().Count > 0;
		//       It's behavior is similar as Subversion:
		//          if you explicitly add one file that is ignored, "cm" will happily accept it and add it,
		//          if you try to add a set of files with a pattern, "cm" will skip the files that are ignored and only add the other ones
		// TODO: provide an updated version of "cm" with a new flag like --applyignorerules
		if (AreAllFiles(InCommand.Files))
		{
			Parameters.Add(TEXT("?"));	// needed only when used with a list of files
		}
		else
		{
			Parameters.Add(TEXT("-R"));	// needed only at the time of workspace creation, to add directories recursively
		}
		if (!GetProvider().IsPartialWorkspace())
		{
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("add"), Parameters, InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		else
		{
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial add"), Parameters, InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		}

		// now update the status of our files
		PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}
	else
	{
		UE_LOGF(LogSourceControl, Warning, "MarkforAdd: No files provided");
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticMarkForAddWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticMarkForAddWorker::UpdateStates);

	// If files have been added directly to a CL, modify the cached state to reflect it (defaults to the Default changelist).
	UpdateChangelistState(GetProvider(), InChangelist, States);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticDeleteWorker::GetName() const
{
	return "Delete";
}

bool FPlasticDeleteWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	if (!GetProvider().IsPartialWorkspace())
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("remove"), TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
	}
	else
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial remove"), TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// now update the status of our files
	PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticDeleteWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteWorkers::UpdateStates);

	// If files have been deleted directly to a CL, modify the cached state to reflect it (defaults to the Default changelist).
	UpdateChangelistState(GetProvider(), InChangelist, States);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticRevertWorker::GetName() const
{
	return "Revert";
}

bool FPlasticRevertWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FRevert, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FRevert>(InCommand.Operation);
	const bool bIsSoftRevert = Operation->IsSoftRevert();
	if (bIsSoftRevert && GetProvider().GetPlasticScmVersion() < PlasticSourceControlVersions::UndoCheckoutKeepChanges)
	{
		// If a soft revert is requested but not supported by the version of Unity Version Control, warn the user and stop
		FText FailureText = FText::FormatOrdered(
			LOCTEXT("PlasticUndoKeepChangesVersionError", "Unity Version Control {0} cannot keep changes when undoing the checkout of the selected files. Update to version {1} or above."),
			FText::FromString(*GetProvider().GetPlasticScmVersion().String),
			FText::FromString(*PlasticSourceControlVersions::UndoCheckoutKeepChanges.String));

		AsyncTask(ENamedThreads::GameThread, [FailureText]
		{
			FMessageLog SourceControlLog("SourceControl");
			SourceControlLog.Error(FailureText);
			SourceControlLog.Notify();
		});

		return false;
	}

	TArray<FString> Files = GetFilesFromCommand(GetProvider(), InCommand);

	TArray<FString> LocallyChangedFiles;
	TArray<FString> CheckedOutFiles;

	int32 i = 0;
	while (i < Files.Num())
	{
		FString& File = Files[i];

		const TSharedRef<const FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(File);

		// Remove files that cannot be reverted from the list (e.g. Private files)
		if (!State->CanRevert())
		{
			// Private files are not under source control, so we can't revert them
			UE_LOGF(LogSourceControl, Log, "File '%ls' cannot be reverted", *File);
			Files.RemoveAt(i);
			continue;
		}
		i++;

		if (State->WorkspaceState == EWorkspaceState::Changed)
		{
			LocallyChangedFiles.Add(State->LocalFilename);
		}
		else
		{
			CheckedOutFiles.Add(State->LocalFilename);
			// in case of a Moved/Renamed, find the rename origin to revert both at once
			if (State->WorkspaceState == EWorkspaceState::Moved)
			{
				const FString& MovedFrom = State->MovedFrom;

				// In case of a file Moved/Renamed, consider the rename Origin (where there is now a Redirector file Added)
				// and add it to the list of files to revert (only if it is not already in) to revert both at once
				if (!CheckedOutFiles.FindByPredicate([&MovedFrom](const FString& File) { return File.Equals(MovedFrom, ESearchCase::IgnoreCase); }))
				{
					CheckedOutFiles.Add(MovedFrom);
				}

				// and delete the Redirector (else the reverted file will collide with it and create a *.private.0 file)
				IFileManager::Get().Delete(*MovedFrom);
			}
			else if (State->WorkspaceState == EWorkspaceState::Added && Operation->ShouldDeleteNewFiles())
			{
				IFileManager::Get().Delete(*File);
			}
		}
	}

	InCommand.bCommandSuccessful = true;

	if (LocallyChangedFiles.Num() > 0)
	{
		if (!GetProvider().IsPartialWorkspace())
		{
			InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("undochange"), TArray<FString>(), LocallyChangedFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		else
		{
			// partial undochange doesn't exist in partial mode
			InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("partial undo"), TArray<FString>(), LocallyChangedFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
	}

	if (CheckedOutFiles.Num() > 0)
	{
		TArray<FString> Parameters;
		if (bIsSoftRevert)
		{
			Parameters.Add(TEXT("--keepchanges"));
		}

		// revert the checkout and any changes of the given file in workspace
		if (!GetProvider().IsPartialWorkspace())
		{
			InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("undocheckout"), Parameters, CheckedOutFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
		else
		{
			InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("partial undocheckout"), Parameters, CheckedOutFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
		}
	}

	// NOTE: optim, in UE4 there was no need to update the status of our files since this is done immediately after by the Editor, except now that we are using changelists
	// update the status of our files: need to check for local changes in case of a SoftRevert
	const PlasticSourceControlUtils::EStatusSearchType SearchType = bIsSoftRevert ? PlasticSourceControlUtils::EStatusSearchType::All : PlasticSourceControlUtils::EStatusSearchType::ControlledOnly;
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(Files, SearchType, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticRevertWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertWorker::UpdateStates);

	// Update affected changelists if any
	for (const FPlasticSourceControlState& NewState : States)
	{
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
		if (State->Changelist.IsInitialized())
		{
			// 1- Remove these files from their previous changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(State->Changelist);
			PreviousChangelist->Files.Remove(State);
			// 2- And reset the reference to their previous changelist
			State->Changelist.Reset();
		}
	}

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}


static TArray<FString> GetFilesCheckedOut(FPlasticSourceControlProvider& InSourceControlProvider, const TArray<FString>& InFiles)
{
	TArray<FString> MoveableFiles;
	MoveableFiles.Reserve(InFiles.Num());
	for (const auto& File : InFiles)
	{
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = InSourceControlProvider.GetStateInternal(File);
		if (FileState->IsCheckedOutImplementation())
		{
			MoveableFiles.Add(File);
		}
		else
		{
			UE_LOGF(LogSourceControl, Warning, "GetFilesCheckedOut: File '%ls' is not checked out and cannot be moved to another changelist", *File);
		}
	}
	return MoveableFiles;
}

FName FPlasticRevertUnchangedWorker::GetName() const
{
	return "RevertUnchanged";
}

bool FPlasticRevertUnchangedWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertUnchangedWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	TArray<FString> Parameters;
	Parameters.Add(TEXT("-R"));

	// Filter out files that we shouldn't try to uncheckout!
	TArray<FString> FilesFromCommand = GetFilesFromCommand(GetProvider(), InCommand);
	TArray<FString> Files = GetFilesCheckedOut(GetProvider(), FilesFromCommand);
	if (Files.Num() == 0 && FilesFromCommand.Num() > 0)
	{
		// If the command specified a set of files but none are checked out, do nothing
		return true;
	}

	// revert the checkout of all unchanged files recursively
	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("uncounchanged"), Parameters, Files, InCommand.InfoMessages, InCommand.ErrorMessages);

	// Now update the status of either the files, or all assets in the Content directory
	if (Files.Num() == 0)
	{
		Files.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
	}
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticRevertUnchangedWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertUnchangedWorker::UpdateStates);

	// Update affected changelists if any
	for (const FPlasticSourceControlState& NewState : States)
	{
		if (!NewState.IsPendingChanges())
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
			if (State->Changelist.IsInitialized())
			{
				// 1- Remove these files from their previous changelist
				TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(State->Changelist);
				PreviousChangelist->Files.Remove(State);
				// 2- And reset the reference to their previous changelist
				State->Changelist.Reset();
			}
		}
	}

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticRevertAllWorker::GetName() const
{
	return "RevertAll";
}

bool FPlasticRevertAllWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertAllWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticRevertAll, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticRevertAll>(InCommand.Operation);

	// Start by updating the Status of all Content, to find all the changes that will be reverted
	{
		TArray<FPlasticSourceControlState> TempStates;
		TArray<FString> ContentDir;
		ContentDir.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
		PlasticSourceControlUtils::RunUpdateStatus(ContentDir, PlasticSourceControlUtils::EStatusSearchType::All, false, InCommand.ErrorMessages, TempStates, InCommand.ChangesetNumber);

		for (auto& State : TempStates)
		{
			if (State.CanRevert())
			{
				if (State.WorkspaceState == EWorkspaceState::Added && Operation->ShouldDeleteNewFiles())
				{
					IFileManager::Get().Delete(*State.GetFilename());
				}

				// Add all modified files to the list of files to be updated (reverted and then reloaded)
				Operation->UpdatedFiles.Add(MoveTemp(State.LocalFilename));

				if (State.WorkspaceState == EWorkspaceState::Moved)
				{
					// In case of a file Moved/Renamed, consider the rename Origin (where there is now a Redirector file Added)
					// and add it to the list of files to revert
					Operation->UpdatedFiles.Add(MoveTemp(State.MovedFrom));

					// and delete the Redirector (else the reverted file will collide with it and create a *.private.0 file)
					IFileManager::Get().Delete(*State.MovedFrom);
				}
			}
		}
	}

	TArray<FString> Results;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--all"));
	// revert the checkout of all files recursively
	if (!GetProvider().IsPartialWorkspace())
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("undocheckout"), Parameters, TArray<FString>(), Results, InCommand.ErrorMessages);
	}
	else
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial undocheckout"), Parameters, TArray<FString>(), Results, InCommand.ErrorMessages);
	}

	// NOTE: don't parse the Results, it has too many quirks, uses the list from the status update;
	// - Renames are not easy to parse without a clean separator in "Origin Destination"
	// - Files added in folders are not accounted for by "undo", only the folder is listed in the results

	// now update the status of the updated files
	if (Operation->UpdatedFiles.Num())
	{
		PlasticSourceControlUtils::InvalidateLocksCache();
		PlasticSourceControlUtils::RunUpdateStatus(Operation->UpdatedFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticRevertAllWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertAllWorker::UpdateStates);

	// Update affected changelists if any
	for (const FPlasticSourceControlState& NewState : States)
	{
		if (!NewState.IsPendingChanges())
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
			if (State->Changelist.IsInitialized())
			{
				// 1- Remove these files from their previous changelist
				TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(State->Changelist);
				PreviousChangelist->Files.Remove(State);
				// 2- And reset the reference to their previous changelist
				State->Changelist.Reset();
			}
		}
	}

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticRevertToRevisionWorker::GetName() const
{
	return "RevertToRevision";
}

bool FPlasticRevertToRevisionWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRevertToRevisionWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticRevertToRevision, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticRevertToRevision>(InCommand.Operation);

	InCommand.bCommandSuccessful = true;
	for (const FString& File : InCommand.Files)
	{
		TArray<FString> Parameters;
		Parameters.Add(FString::Printf(TEXT("\"%s#cs:%d\""), *File, Operation->ChangesetId));

		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("revert"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	if (InCommand.bCommandSuccessful)
	{
		Operation->UpdatedFiles = InCommand.Files;
	}

	// now update the status of our files
	PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticRevertToRevisionWorker::UpdateStates()
{
	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticMakeWorkspaceWorker::GetName() const
{
	return "MakeWorkspace";
}

bool FPlasticMakeWorkspaceWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticMakeWorkspaceWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticMakeWorkspace, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticMakeWorkspace>(InCommand.Operation);

	{
		TArray<FString> Parameters;
		Parameters.Add(Operation->ServerUrl);
		Parameters.Add(Operation->RepositoryName);
		// Note: the whole operation should fail entirely if the repository creation failed (if the repository already exists, if the organization name is invalid, credential, autorizations etc.)
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("makerepository"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
		// Specifically detect the specific error when the organization name is invalid, and add an more human readable message.
		if (InCommand.ErrorMessages.Contains(TEXT("Can't resolve DNS entry for cloud.plasticscm.com")))
		{
			InCommand.ErrorMessages.Add(TEXT("Invalid cloud organization name."));
		}
	}
	if (InCommand.bCommandSuccessful)
	{
		TArray<FString> Parameters;
		Parameters.Add(Operation->WorkspaceName);
		Parameters.Add(TEXT(".")); // current path, ie. ProjectDir
		Parameters.Add(FString::Printf(TEXT("--repository=rep:%s@repserver:%s"), *Operation->RepositoryName, *Operation->ServerUrl));
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("makeworkspace"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
	}
	if (InCommand.bCommandSuccessful && Operation->bPartialWorkspace)
	{
		TArray<FString> Parameters;
		Parameters.Add(TEXT("update"));
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticMakeWorkspaceWorker::UpdateStates()
{
	return false;
}

FName FPlasticSwitchToPartialWorkspaceWorker::GetName() const
{
	return "SwitchToPartialWorkspace";
}

bool FPlasticSwitchToPartialWorkspaceWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSwitchToPartialWorkspaceWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	{
		TArray<FString> Parameters;
		Parameters.Add(TEXT("update"));
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// Update the workspace to set the changeset number to -1 if all went well
	{
		TArray<FString> ProjectFiles;
		ProjectFiles.Add(FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()));
		PlasticSourceControlUtils::RunUpdateStatus(ProjectFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticSwitchToPartialWorkspaceWorker::UpdateStates()
{
	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticGetLocksWorker::GetName() const
{
	return "GetLocks";
}

bool FPlasticGetLocksWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetLocksWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticGetLocks, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticGetLocks>(InCommand.Operation);

	{
		// In the View Locks window, always show locks for all destination branches
		const bool bForAllDestBranches = true;
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunListLocks(GetProvider(), bForAllDestBranches, Operation->Locks);
	}

	{
		FString RepositoryName, ServerUrl;
		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::GetWorkspaceInfo(InCommand.WorkspaceSelector, InCommand.BranchName, RepositoryName, ServerUrl, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetLocksWorker::UpdateStates()
{
	return false;
}

FName FPlasticUnlockWorker::GetName() const
{
	return "Unlock";
}

bool FPlasticUnlockWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticUnlockWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticUnlock, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticUnlock>(InCommand.Operation);

	InCommand.bCommandSuccessful = true;

	TArray<FString> Branches;
	for (const FPlasticSourceControlLockRef& Lock : Operation->Locks)
	{
		Branches.AddUnique(Lock->Branch);
	}
	for (const FString& Branch : Branches)
	{
		TArray<FString> Parameters;
		Parameters.Add(TEXT("unlock"));
		if (Operation->bRemove)
			Parameters.Add(TEXT("--remove"));

		Parameters.Add(FString::Printf(TEXT("--branch=%s"), *Branch));
		for (const FPlasticSourceControlLockRef& Lock : Operation->Locks)
		{
			if (Lock->Branch == Branch)
			{
				Parameters.Add(FString::Printf(TEXT("itemid:%d "), Lock->ItemId));
			}
		}
		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunCommand(TEXT("lock"), Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// now update the status of our files
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticUnlockWorker::UpdateStates()
{
	PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
	// Note: Force the return to always trigger a refresh of the List of Locks
	// It's needed since when removing the Lock of a Checked Out asset there is actually no change in local status as the asset remains Checked Out!
	return true;
}

FName FPlasticGetBranchesWorker::GetName() const
{
	return "GetBranches";
}

bool FPlasticGetBranchesWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetBranchesWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticGetBranches, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticGetBranches>(InCommand.Operation);

	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetBranches(Operation->FromDate, Operation->Branches, InCommand.ErrorMessages);
	}

	{
		FString RepositoryName, ServerUrl;
		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::GetWorkspaceInfo(InCommand.WorkspaceSelector, InCommand.BranchName, RepositoryName, ServerUrl, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetBranchesWorker::UpdateStates()
{
	return false;
}


FName FPlasticSwitchWorker::GetName() const
{
	return "Switch";
}

bool FPlasticSwitchWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSwitchWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticSwitch>(InCommand.Operation);

	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunSwitch(Operation->BranchName, Operation->ChangesetId, GetProvider().IsPartialWorkspace(), Operation->UpdatedFiles, InCommand.ErrorMessages);

	// the current branch is used to asses the status of Retained Locks
	{
		FString RepositoryName, ServerUrl;
		PlasticSourceControlUtils::GetWorkspaceInfo(InCommand.WorkspaceSelector, InCommand.BranchName, RepositoryName, ServerUrl, InCommand.ErrorMessages);
		GetProvider().SetWorkspaceSelector(InCommand.WorkspaceSelector, InCommand.BranchName);
		PlasticSourceControlUtils::InvalidateLocksCache();
	}

	// now update the status of the updated files
	if (InCommand.bCommandSuccessful && Operation->UpdatedFiles.Num())
	{
		PlasticSourceControlUtils::RunUpdateStatus(Operation->UpdatedFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticSwitchWorker::UpdateStates()
{
	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}


FName FPlasticMergeBranchWorker::GetName() const
{
	return "Merge";
}

bool FPlasticMergeBranchWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticMergeBranchWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticMergeBranch, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticMergeBranch>(InCommand.Operation);

	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunMergeBranch(Operation->BranchName, Operation->UpdatedFiles, InCommand.ErrorMessages);

	// now update the status of the updated files
	if (Operation->UpdatedFiles.Num())
	{
		PlasticSourceControlUtils::InvalidateLocksCache();
		PlasticSourceControlUtils::RunUpdateStatus(Operation->UpdatedFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticMergeBranchWorker::UpdateStates()
{
	// Update the Default changelist.
	UpdateChangelistState(GetProvider(), FPlasticSourceControlChangelist::DefaultChangelist, States);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}


FName FPlasticCreateBranchWorker::GetName() const
{
	return "CreateBranch";
}

bool FPlasticCreateBranchWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCreateBranchWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticCreateBranch, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticCreateBranch>(InCommand.Operation);

	return PlasticSourceControlUtils::RunCreateBranch(Operation->BranchName, Operation->Comment, InCommand.ErrorMessages);
}

bool FPlasticCreateBranchWorker::UpdateStates()
{
	return false;
}


FName FPlasticRenameBranchWorker::GetName() const
{
	return "RenameBranch";
}

bool FPlasticRenameBranchWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticRenameBranchWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticRenameBranch, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticRenameBranch>(InCommand.Operation);

	return PlasticSourceControlUtils::RunRenameBranch(Operation->OldName, Operation->NewName, InCommand.ErrorMessages);
}

bool FPlasticRenameBranchWorker::UpdateStates()
{
	return false;
}


FName FPlasticDeleteBranchesWorker::GetName() const
{
	return "DeleteBranches";
}

bool FPlasticDeleteBranchesWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteBranchesWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticDeleteBranches, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticDeleteBranches>(InCommand.Operation);

	return PlasticSourceControlUtils::RunDeleteBranches(Operation->BranchNames, InCommand.ErrorMessages);
}

bool FPlasticDeleteBranchesWorker::UpdateStates()
{
	return false;
}


FName FPlasticGetChangesetsWorker::GetName() const
{
	return "GetChangesets";
}

bool FPlasticGetChangesetsWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetChangesetsWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticGetChangesets, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticGetChangesets>(InCommand.Operation);

	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetChangesets(Operation->FromDate, Operation->Changesets, InCommand.ErrorMessages);
	}

	{
		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::GetChangesetNumber(InCommand.ChangesetNumber, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetChangesetsWorker::UpdateStates()
{
	return false;
}



FName FPlasticGetChangesetFilesWorker::GetName() const
{
	return "GetChangesetFiles";
}

bool FPlasticGetChangesetFilesWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetChangesetFilesWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FPlasticGetChangesetFiles, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FPlasticGetChangesetFiles>(InCommand.Operation);

	if (!Operation->Changeset.IsValid())
	{
		return false;
	}

	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetChangesetFiles(Operation->Changeset.ToSharedRef(), Operation->Files, InCommand.ErrorMessages);
	}

	{
		InCommand.bCommandSuccessful &= PlasticSourceControlUtils::GetChangesetNumber(InCommand.ChangesetNumber, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetChangesetFilesWorker::UpdateStates()
{
	return false;
}


FName FPlasticUpdateStatusWorker::GetName() const
{
	return "UpdateStatus";
}

bool FPlasticUpdateStatusWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticUpdateStatusWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FUpdateStatus>(InCommand.Operation);

	// Note: ShouldCheckAllFiles is never set to true (SetCheckingAllFiles)
	UE_LOGF(LogSourceControl, Log, "status of %d items (ShouldUpdateHistory=%d, ShouldGetOpenedOnly=%d, ShouldUpdateModifiedState=%d)",
		InCommand.Files.Num(), Operation->ShouldUpdateHistory(), Operation->ShouldGetOpenedOnly(), Operation->ShouldUpdateModifiedState());

	const TArray<FString> Files = GetFilesFromCommand(GetProvider(), InCommand);

	if (Files.Num() > 0)
	{
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunUpdateStatus(Files, PlasticSourceControlUtils::EStatusSearchType::All, Operation->ShouldUpdateHistory(), InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
		// Remove all "is not in a workspace" error and convert the result to "success" if there are no other errors
		PlasticSourceControlUtils::RemoveRedundantErrors(InCommand, TEXT("is not in a workspace."));
		if (!InCommand.bCommandSuccessful)
		{
			// In case of error, execute a 'checkconnection' command to check the connection to the server.
			UE_LOGF(LogSourceControl, Warning, "Error on 'status', execute a 'checkconnection' to test the connection to the server");
			InCommand.bConnectionDropped = !PlasticSourceControlUtils::RunCheckConnection(InCommand.WorkspaceSelector, InCommand.BranchName, InCommand.RepositoryName, InCommand.ServerUrl, InCommand.InfoMessages, InCommand.ErrorMessages);
			return false;
		}

		if (Operation->ShouldUpdateHistory())
		{
			// Get the history of the files (on all branches)
			InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunGetHistory(Operation->ShouldUpdateHistory(), States, InCommand.ErrorMessages);
		}
		else
		{
			FPlasticSourceControlSettings& PlasticSettings = GetProvider().AccessSettings();
			if (PlasticSettings.GetUpdateStatusOtherBranches() && AreAllFiles(Files))
			{
				// Get only the last revision of the files (checking all branches)
				// in order to warn the user if the file has been changed on another branch
				InCommand.bCommandSuccessful &= PlasticSourceControlUtils::RunGetHistory(Operation->ShouldUpdateHistory(), States, InCommand.ErrorMessages);
			}
		}
	}
	// no path provided: only update the status of assets in Content/ directory if requested
	// Perforce "opened files" are those that have been modified (or added/deleted): that is what we get with a simple status from the root
	// This is called by the "CheckOut" Content Browser filter as well as our source control Refresh menu.
	else if (Operation->ShouldGetOpenedOnly())
	{
		TArray<FString> ProjectDirs;
		ProjectDirs.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunUpdateStatus(ProjectDirs, PlasticSourceControlUtils::EStatusSearchType::All, Operation->ShouldUpdateHistory(), InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}
	else
	{
		// Note: workaround for the case of submitting a changelist, calling UpdateStatus with no files nor the changelist.
		// No consequences, and no way to fix it, so let's not show an error.
		InCommand.bCommandSuccessful = true;
	}

	// TODO: re-evaluate how to optimize this heavy operation using some of these hints flags
	// - ShouldGetOpenedOnly hint would be to call for all a whole workspace status update
	// - ShouldUpdateModifiedState hint not used as the above normal status has already told us this information (like Git and Mercurial)

	return InCommand.bCommandSuccessful;
}

bool FPlasticUpdateStatusWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticUpdateStatusWorker::UpdateStates);

	// Update affected changelists if any (in case of a file reverted outside of the Unreal Editor)
	for (const FPlasticSourceControlState& NewState : States)
	{
		if (!NewState.IsPendingChanges())
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
			if (State->Changelist.IsInitialized())
			{
				// 1- Remove these files from their previous changelist
				TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(State->Changelist);
				PreviousChangelist->Files.Remove(State);
				// 2- And reset the reference to their previous changelist
				State->Changelist.Reset();
			}
		}
		else if (NewState.WorkspaceState == EWorkspaceState::Changed)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = GetProvider().GetStateInternal(NewState.GetFilename());
			if (State->WorkspaceState != EWorkspaceState::Changed)
			{
				// 1- Add these files to the default changelist
				TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> DefaultChangelist = GetProvider().GetStateInternal(FPlasticSourceControlChangelist::DefaultChangelist);
				DefaultChangelist->Files.AddUnique(State);
				// 2- And set the reference to the default changelist
				State->Changelist = FPlasticSourceControlChangelist::DefaultChangelist;
			}
		}
	}

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticCopyWorker::GetName() const
{
	return "Copy";
}

bool FPlasticCopyWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCopyWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FCopy, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCopy>(InCommand.Operation);

	if (InCommand.Files.Num() == 1)
	{
		const FString& Origin = InCommand.Files[0];
		const FString Destination = FPaths::ConvertRelativePathToFull(Operation->GetDestination());

		// Branch: The new file is branched from the original file (vs Add: The new file has no relation to the original file)
		const bool bIsMoveOperation = (Operation->CopyMethod == FCopy::ECopyMethod::Branch);
		if (bIsMoveOperation)
		{
			UE_LOGF(LogSourceControl, Log, "Moving %ls to %ls...", *Origin, *Destination);
			// In case of rename, we have to undo what the Editor (created a redirector and added the dest asset), and then redo it with Unity Version Control
			// - revert the 'cm add' that was applied to the destination by the Editor
			{
				TArray<FString> DestinationFiles;
				DestinationFiles.Add(Destination);
				InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("undochange"), TArray<FString>(), DestinationFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
			}
			// - execute a 'cm move --nomoveondisk' command to the destination to tell cm what happened
			if (InCommand.bCommandSuccessful)
			{
				TArray<FString> Parameters;
				Parameters.Add(TEXT("--nomoveondisk"));
				TArray<FString> Files;
				Files.Add(Origin);
				Files.Add(Destination);
				if (!GetProvider().IsPartialWorkspace())
				{
					InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("move"), Parameters, Files, InCommand.InfoMessages, InCommand.ErrorMessages);
				}
				else
				{
					InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial move"), Parameters, Files, InCommand.InfoMessages, InCommand.ErrorMessages);
				}
			}
			// - add the redirector file (if it exists) to source control
			if (InCommand.bCommandSuccessful)
			{
				TArray<FString> Files;
				Files.Add(Origin);
				if (!GetProvider().IsPartialWorkspace())
				{
					InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("add"), TArray<FString>(), Files, InCommand.InfoMessages, InCommand.ErrorMessages);
				}
				else
				{
					InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("partial add"), TArray<FString>(), Files, InCommand.InfoMessages, InCommand.ErrorMessages);
				}
			}
		}
		else
		{
			// copy operation: destination file already added to Source Control, and original asset not changed, so nothing to do
			InCommand.bCommandSuccessful = true;
		}

		// now update the status of our files
		TArray<FString> BothFiles;
		BothFiles.Add(Origin);
		BothFiles.Add(Destination);
		PlasticSourceControlUtils::InvalidateLocksCache();
		PlasticSourceControlUtils::RunUpdateStatus(BothFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Copy is working for one file only: %d provided!", InCommand.Files.Num());
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticCopyWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticCopyWorker::UpdateStates);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticSyncWorker::GetName() const
{
	return "Sync";
}

bool FPlasticSyncWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSyncWorker::Execute);

	TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> SyncOperation = StaticCastSharedRef<FPlasticSyncAll>(InCommand.Operation);

	TArray<FString> UpdatedFiles;
	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunUpdate(InCommand.Files, GetProvider().IsPartialWorkspace(), SyncOperation->GetRevision(), UpdatedFiles, InCommand.ErrorMessages);

	// the current branch is used to asses the status of Retained Locks
	{
		FString RepositoryName, ServerUrl;
		PlasticSourceControlUtils::GetWorkspaceInfo(InCommand.WorkspaceSelector, InCommand.BranchName, RepositoryName, ServerUrl, InCommand.ErrorMessages);
		GetProvider().SetWorkspaceSelector(InCommand.WorkspaceSelector, InCommand.BranchName);
	}

	// now update the status of the updated files
	if (InCommand.bCommandSuccessful && UpdatedFiles.Num())
	{
		PlasticSourceControlUtils::RunUpdateStatus(UpdatedFiles, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);
	}

	if ((InCommand.Operation->GetName() == FName("SyncAll")))
	{
		TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> SyncAllOperation = StaticCastSharedRef<FPlasticSyncAll>(InCommand.Operation);
		SyncAllOperation->UpdatedFiles = MoveTemp(UpdatedFiles);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticSyncWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSyncWorker::UpdateStates);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticResolveWorker::GetName() const
{
	return "Resolve";
}

bool FPlasticResolveWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticResolveWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	// Currently resolve operation is always on one file only, but the following would works for many
	for (const FString& File : InCommand.Files)
	{
		auto State = GetProvider().GetStateInternal(File);

		// To resolve the conflict, merge the file by keeping it like it is on file system
		// TODO: according to documentation, this cannot work for cherry-picking
		// merge cs:2@repo@url:port --merge --keepdestination "/path/to/file"

		// Use Merge Parameters obtained in the UpdateStatus operation
		TArray<FString> Parameters = State->PendingMergeParameters;
		Parameters.Add(TEXT("--merge"));
		Parameters.Add(TEXT("--keepdestination"));

		TArray<FString> OneFile;
		OneFile.Add(State->PendingResolveInfo.BaseFile);

		UE_LOGF(LogSourceControl, Log, "resolve %ls", *State->PendingResolveInfo.BaseFile);
		// Mark the conflicted file as resolved
		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("merge"), Parameters, OneFile, InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// now update the status of our files
	PlasticSourceControlUtils::InvalidateLocksCache();
	PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

	return InCommand.bCommandSuccessful;
}

bool FPlasticResolveWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticResolveWorker::UpdateStates);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}


FName FPlasticGetPendingChangelistsWorker::GetName() const
{
	return "UpdateChangelistsStatus";
}

bool FPlasticGetPendingChangelistsWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetPendingChangelistsWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FUpdatePendingChangelistsStatus>(InCommand.Operation);

	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetChangelists(OutChangelistsStates, OutCLFilesStates, InCommand.ErrorMessages);
	if (InCommand.bCommandSuccessful)
	{
		// Remove the changelist that were not requested by the user.
		if (!Operation->ShouldUpdateAllChangelists())
		{
			const TArray<FSourceControlChangelistRef>& RequestedChangelists = Operation->GetChangelistsToUpdate();
			OutChangelistsStates.RemoveAll([&RequestedChangelists](const FPlasticSourceControlChangelistState& ChangelistState)
				{
					FPlasticSourceControlChangelistRef RemoveChangelistCandidate = StaticCastSharedRef<FPlasticSourceControlChangelist>(ChangelistState.GetChangelist());
					return !RequestedChangelists.ContainsByPredicate([&RemoveChangelistCandidate](const FSourceControlChangelistRef& Requested)
						{
							return StaticCastSharedRef<FPlasticSourceControlChangelist>(Requested)->GetName() == RemoveChangelistCandidate->GetName();
						});
				});
		}

		InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetShelves(OutChangelistsStates, InCommand.ErrorMessages);
	}

	bCleanupCache = InCommand.bCommandSuccessful;

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetPendingChangelistsWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetPendingChangelistsWorker::UpdateStates);

	bool bUpdated = false;

	const FDateTime Now = FDateTime::Now();

	// first update cached state from 'changes' call
	for (int32 StatusIndex = 0; StatusIndex < OutChangelistsStates.Num(); StatusIndex++)
	{
		const FPlasticSourceControlChangelistState& CLStatus = OutChangelistsStates[StatusIndex];
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(CLStatus.Changelist);
		// Timestamp is used to throttle status requests, so update it to current time:
		*ChangelistState = CLStatus;
		ChangelistState->TimeStamp = Now;
		bUpdated = true;

		// Update files in the changelist
		bool bUpdateFilesStates = (OutCLFilesStates.Num() == OutChangelistsStates.Num());
		if (bUpdateFilesStates)
		{
			ChangelistState->Files.Reset(OutCLFilesStates[StatusIndex].Num());
			for (const auto& FileState : OutCLFilesStates[StatusIndex])
			{
				TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> CachedFileState = GetProvider().GetStateInternal(FileState.LocalFilename);
				if (CachedFileState->WorkspaceState == EWorkspaceState::Unknown)
				{
					CachedFileState->WorkspaceState = FileState.WorkspaceState;
					CachedFileState->TimeStamp = Now;
				}
				CachedFileState->Changelist = CLStatus.Changelist;
				ChangelistState->Files.AddUnique(CachedFileState);
			}
		}
	}

	if (bCleanupCache)
	{
		TArray<FPlasticSourceControlChangelist> ChangelistsToRemove;
		GetProvider().GetCachedStateByPredicate([this, &ChangelistsToRemove](const FSourceControlChangelistStateRef& InCLState) {
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> CLState = StaticCastSharedRef<FPlasticSourceControlChangelistState>(InCLState);

			if (Algo::NoneOf(OutChangelistsStates, [&CLState](const FPlasticSourceControlChangelistState& UpdatedCLState) {
					return CLState->Changelist == UpdatedCLState.Changelist;
				}))
			{
				ChangelistsToRemove.Add(CLState->Changelist);
			}

			return false;
			});

		for (const FPlasticSourceControlChangelist& ChangelistToRemove : ChangelistsToRemove)
		{
			GetProvider().RemoveChangelistFromCache(ChangelistToRemove);
		}
	}

	return bUpdated;
}

static FPlasticSourceControlChangelist GenerateUniqueChangelistName(FPlasticSourceControlProvider& PlasticSourceControlProvider)
{
	FPlasticSourceControlChangelist NewChangelist;

	// Generate a unique number for the name of the new changelist: start from current changeset and increment until a number is available as a new changelist number
	int32 ChangelistNumber = PlasticSourceControlProvider.GetChangesetNumber();
	bool bNewNumberOk = false;
	do
	{
		ChangelistNumber++;
		NewChangelist = FPlasticSourceControlChangelist(FString::FromInt(ChangelistNumber));
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = PlasticSourceControlProvider.GetStateInternal(NewChangelist);
		bNewNumberOk = !ChangelistState->Changelist.IsInitialized();
	} while (!bNewNumberOk);
	NewChangelist.SetInitialized();

	return NewChangelist;
}

static FPlasticSourceControlChangelist CreatePendingChangelist(FPlasticSourceControlProvider& PlasticSourceControlProvider, const FString& InDescription, TArray<FString>& InInfoMessages, TArray<FString>& InErrorMessages)
{
	FPlasticSourceControlChangelist NewChangelist = GenerateUniqueChangelistName(PlasticSourceControlProvider);

	bool bCommandSuccessful;
	TArray<FString> Parameters;
	if (PlasticSourceControlProvider.GetPlasticScmVersion() < PlasticSourceControlVersions::NewChangelistFileArgs)
	{
		Parameters.Add(TEXT("add"));
		Parameters.Add(TEXT("\"") + NewChangelist.GetName() + TEXT("\""));
		Parameters.Add(TEXT("\"") + InDescription + TEXT("\""));
		Parameters.Add(TEXT("--persistent")); // Create a persistent changelist to stay close to Perforce behavior
		bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, TArray<FString>(), InInfoMessages, InErrorMessages);
	}
	else
	{
		Parameters.Add(TEXT("create"));
		const FScopedTempFile ChangelistNameFile(NewChangelist.GetName());
		Parameters.Add(FString::Printf(TEXT("--namefile=\"%s\""), *ChangelistNameFile.GetFilename()));
		const FScopedTempFile ChangelistDescriptionFile(InDescription);
		Parameters.Add(FString::Printf(TEXT("--descriptionfile=\"%s\""), *ChangelistDescriptionFile.GetFilename()));
		Parameters.Add(TEXT("--persistent")); // Create a persistent changelist to stay close to Perforce behavior
		UE_LOGF(LogSourceControl, Verbose, "CreatePendingChangelist(%ls):\n\"%ls\"", *NewChangelist.GetName(), *InDescription);
		bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, TArray<FString>(), InInfoMessages, InErrorMessages);
	}
	if (!bCommandSuccessful)
	{
		NewChangelist.Reset();
	}

	return NewChangelist;
}

static bool EditChangelistDescription(const FPlasticSourceControlProvider& PlasticSourceControlProvider, const FPlasticSourceControlChangelist& InChangelist, const FString& InDescription, TArray<FString>& InInfoMessages, TArray<FString>& InErrorMessages)
{
	TArray<FString> Parameters;
	Parameters.Add(TEXT("edit"));
	if (PlasticSourceControlProvider.GetPlasticScmVersion() < PlasticSourceControlVersions::NewChangelistFileArgs)
	{
		Parameters.Add(TEXT("\"") + InChangelist.GetName() + TEXT("\""));
		Parameters.Add(TEXT("description"));
		Parameters.Add(TEXT("\"") + InDescription + TEXT("\""));
		return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, TArray<FString>(), InInfoMessages, InErrorMessages);
	}
	else
	{
		const FScopedTempFile ChangelistNameFile(InChangelist.GetName());
		Parameters.Add(FString::Printf(TEXT("--namefile=\"%s\""), *ChangelistNameFile.GetFilename()));
		Parameters.Add(TEXT("description"));
		const FScopedTempFile ChangelistDescriptionFile(InDescription);
		Parameters.Add(FString::Printf(TEXT("--descriptionfile=\"%s\""), *ChangelistDescriptionFile.GetFilename()));
		UE_LOGF(LogSourceControl, Verbose, "EditChangelistDescription(%ls\n%ls)", *InChangelist.GetName(), *InDescription);
		return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, TArray<FString>(), InInfoMessages, InErrorMessages);
	}
}

static bool MoveFilesToChangelist(const FPlasticSourceControlProvider& InSourceControlProvider, const FPlasticSourceControlChangelist& InChangelist, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	if (InFiles.Num() > 0)
	{
		TArray<FString> Parameters;
		if (InSourceControlProvider.GetPlasticScmVersion() < PlasticSourceControlVersions::NewChangelistFileArgs)
		{
			Parameters.Add(TEXT("\"") + InChangelist.GetName() + TEXT("\""));
			Parameters.Add(TEXT("add"));
			return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, InFiles, OutResults, OutErrorMessages);
		}
		else
		{
			const FScopedTempFile ChangelistNameFile(InChangelist.GetName());
			Parameters.Add(FString::Printf(TEXT("--namefile=\"%s\""), *ChangelistNameFile.GetFilename()));
			Parameters.Add(TEXT("add"));
			UE_LOGF(LogSourceControl, Verbose, "MoveFilesToChangelist: %ls", *InChangelist.GetName());
			return PlasticSourceControlUtils::RunCommand(TEXT("changelist"), Parameters, InFiles, OutResults, OutErrorMessages);
		}
	}
	return true;
}

// Old "cm" doesn't support newlines, quotes, and question marks on changelist's name or description
static FString CleanupChangelistDescription(const FPlasticSourceControlProvider& InSourceControlProvider, const FText& InDescription)
{
	FString Description = InDescription.ToString();
	if (InSourceControlProvider.GetPlasticScmVersion() < PlasticSourceControlVersions::NewChangelistFileArgs)
	{
		Description.ReplaceInline(TEXT("\r\n"), TEXT(" "), ESearchCase::CaseSensitive);
		Description.ReplaceCharInline(TEXT('\n'), TEXT(' '), ESearchCase::CaseSensitive);
		Description.ReplaceCharInline(TEXT('\"'), TEXT('\''), ESearchCase::CaseSensitive);
		Description.ReplaceCharInline(TEXT('?'), TEXT('.'), ESearchCase::CaseSensitive);
		Description.ReplaceCharInline(TEXT('*'), TEXT('.'), ESearchCase::CaseSensitive);
	}
	return Description;
}

FPlasticNewChangelistWorker::FPlasticNewChangelistWorker(FPlasticSourceControlProvider& InSourceControlProvider)
	: IPlasticSourceControlWorker(InSourceControlProvider)
	, NewChangelistState(NewChangelist)
{
}

FName FPlasticNewChangelistWorker::GetName() const
{
	return "NewChangelist";
}

bool FPlasticNewChangelistWorker::Execute(class FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticNewChangelistWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FNewChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FNewChangelist>(InCommand.Operation);
	FString Description = CleanupChangelistDescription(GetProvider(), Operation->GetDescription());

	// Create a new numbered persistent changelist ala Perforce
	NewChangelist = CreatePendingChangelist(GetProvider(), Description, InCommand.InfoMessages, InCommand.ErrorMessages);

	// Successfully created new changelist
	if (NewChangelist.IsInitialized())
	{
		InCommand.bCommandSuccessful = true;

		NewChangelistState.Changelist = NewChangelist;
		NewChangelistState.Description = MoveTemp(Description);

		Operation->SetNewChangelist(MakeShared<FPlasticSourceControlChangelist>(NewChangelist));

		if (InCommand.Files.Num() > 0)
		{
			// Filter out files that cannot be moved from the Default changelist; files with Pending Changes but not actually Checked Out
			TArray<FString> FilesToMove = GetFilesCheckedOut(GetProvider(), InCommand.Files);
			if (FilesToMove.Num() == 0)
			{
				UE_LOGF(LogSourceControl, Error, "No file can be moved.");
				InCommand.bCommandSuccessful = false;
				return InCommand.bCommandSuccessful;
			}

			InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), NewChangelist, FilesToMove, InCommand.InfoMessages, InCommand.ErrorMessages);
			if (InCommand.bCommandSuccessful)
			{
				MovedFiles = MoveTemp(FilesToMove);
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticNewChangelistWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticNewChangelistWorker::UpdateStates);

	if (NewChangelist.IsInitialized())
	{
		const FDateTime Now = FDateTime::Now();

		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(NewChangelist);
		*ChangelistState = NewChangelistState;
		ChangelistState->TimeStamp = Now;

		// 3 things to do here:
		for (const FString& MovedFile : MovedFiles)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(MovedFile);

			// 1- Remove these files from their previous changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(FileState->Changelist);
			PreviousChangelist->Files.Remove(FileState);

			// 2- Add to the new changelist
			ChangelistState->Files.Add(FileState);

			// 3- Update changelist in file state
			FileState->Changelist = NewChangelist;
			FileState->TimeStamp = Now;
		}

		return true;
	}
	else
	{
		return false;
	}
}


FName FPlasticDeleteChangelistWorker::GetName() const
{
	return "DeleteChangelist";
}

bool FPlasticDeleteChangelistWorker::Execute(class FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteChangelistWorker::Execute);

	// Can't delete the default changelist
	if (InCommand.Changelist.IsDefault())
	{
		InCommand.bCommandSuccessful = false;
	}
	else
	{
		check(InCommand.Operation->GetName() == GetName());
		TSharedRef<FDeleteChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FDeleteChangelist>(InCommand.Operation);

		InCommand.bCommandSuccessful = DeleteChangelist(GetProvider(), InCommand.Changelist, InCommand.InfoMessages, InCommand.ErrorMessages);

		// NOTE: for now it is not possible to delete a changelist with files through the Editor
		if (InCommand.Files.Num() > 0 && InCommand.bCommandSuccessful)
		{
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);
			const TArray<FString> Files = FileNamesFromFileStates(ChangelistState->Files);
			InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), FPlasticSourceControlChangelist::DefaultChangelist, Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		}

		// Keep track of changelist to update the cache
		if (InCommand.bCommandSuccessful)
		{
			DeletedChangelist = InCommand.Changelist;
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticDeleteChangelistWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteChangelistWorker::UpdateStates);

	if (DeletedChangelist.IsInitialized())
	{
		return GetProvider().RemoveChangelistFromCache(DeletedChangelist);
	}

	return false;
}


FName FPlasticEditChangelistWorker::GetName() const
{
	return "EditChangelist";
}

bool FPlasticEditChangelistWorker::Execute(class FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticEditChangelistWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FEditChangelist, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FEditChangelist>(InCommand.Operation);

	EditedDescription = CleanupChangelistDescription(GetProvider(), Operation->GetDescription());

	if (InCommand.Changelist.IsDefault())
	{
		// Create a new numbered persistent changelist since we cannot edit the default changelist
		EditedChangelist = CreatePendingChangelist(GetProvider(), EditedDescription, InCommand.InfoMessages, InCommand.ErrorMessages);
		if (EditedChangelist.IsInitialized())
		{
			// And then move all its files to the new changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);
			// Filter out files that cannot be moved from the Default changelist; files with Pending Changes but not actually Checked Out
			TArray<FString> FilesToMove = GetFilesCheckedOut(GetProvider(), FileNamesFromFileStates(ChangelistState->Files));
			if (FilesToMove.Num() == 0)
			{
				UE_LOGF(LogSourceControl, Error, "No file can be moved.");
				InCommand.bCommandSuccessful = false;
				return InCommand.bCommandSuccessful;
			}

			InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), EditedChangelist, FilesToMove, InCommand.InfoMessages, InCommand.ErrorMessages);
			if (InCommand.bCommandSuccessful)
			{
				ReopenedFiles = MoveTemp(FilesToMove);
			}
		}
	}
	else
	{
		InCommand.bCommandSuccessful = EditChangelistDescription(GetProvider(), InCommand.Changelist, EditedDescription, InCommand.InfoMessages, InCommand.ErrorMessages);
		if (InCommand.bCommandSuccessful)
		{
			EditedChangelist = InCommand.Changelist;
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticEditChangelistWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticEditChangelistWorker::UpdateStates);

	if (EditedChangelist.IsInitialized())
	{
		const FDateTime Now = FDateTime::Now();
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> EditedChangelistState = GetProvider().GetStateInternal(EditedChangelist);
		EditedChangelistState->Description = EditedDescription;
		EditedChangelistState->Changelist = EditedChangelist;
		EditedChangelistState->TimeStamp = Now;

		// 3 things to do here:
		for (const FString& ReopenedFile : ReopenedFiles)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(ReopenedFile);

			// 1- Remove these files from their previous changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(FileState->Changelist);
			PreviousChangelist->Files.Remove(FileState);

			// 2- Add to the new changelist
			EditedChangelistState->Files.Add(FileState);

			// 3- Update changelist in file state
			FileState->Changelist = EditedChangelist;
			FileState->TimeStamp = Now;
		}

		return true;
	}
	else
	{
		return false;
	}
}


FName FPlasticReopenWorker::GetName() const
{
	return "MoveToChangelist";
}

bool FPlasticReopenWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticReopenWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());

	// Filter out files that cannot be moved from the Default changelist; files with Pending Changes but not actually Checked Out
	TArray<FString> FilesToMove = GetFilesCheckedOut(GetProvider(), InCommand.Files);
	if (FilesToMove.Num() == 0)
	{
		UE_LOGF(LogSourceControl, Error, "No file can be moved.");
		InCommand.bCommandSuccessful = false;
		return InCommand.bCommandSuccessful;
	}

	InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), InCommand.Changelist, FilesToMove, InCommand.InfoMessages, InCommand.ErrorMessages);
	if (InCommand.bCommandSuccessful)
	{
		ReopenedFiles = MoveTemp(FilesToMove);
		DestinationChangelist = InCommand.Changelist;
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticReopenWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticReopenWorker::UpdateStates);

	if (DestinationChangelist.IsInitialized())
	{
		const FDateTime Now = FDateTime::Now();
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> DestinationChangelistState = GetProvider().GetStateInternal(DestinationChangelist);

		// 3 things to do here:
		for (const FString& ReopenedFile : ReopenedFiles)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(ReopenedFile);

			// 1- Remove these files from their previous changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> PreviousChangelist = GetProvider().GetStateInternal(FileState->Changelist);
			PreviousChangelist->Files.Remove(FileState);

			// 2- Add to the new changelist
			DestinationChangelistState->Files.Add(FileState);

			// 3- Update changelist in file state
			FileState->Changelist = DestinationChangelist;
			FileState->TimeStamp = Now;
		}

		return ReopenedFiles.Num() > 0;
	}
	else
	{
		return false;
	}
}

static bool CreateShelve(const FString& InChangelistName, const FString& InChangelistDescription, const TArray<FString>& InFilesToShelve, int32& OutShelveId, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Results;
	TArray<FString> Parameters;
	const FString ShelveDescription = FString::Printf(TEXT("Changelist%s: %s"), *InChangelistName, *InChangelistDescription);
	const FScopedTempFile CommentsFile(ShelveDescription);
	Parameters.Add(TEXT("create"));
	Parameters.Add(FString::Printf(TEXT("-commentsfile=\"%s\""), *CommentsFile.GetFilename()));
	const bool bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("shelveset"), Parameters, InFilesToShelve, Results, OutErrorMessages);
	if (bCommandSuccessful && Results.Num() > 0)
	{
		// Parse the result to update the changelist state with the id of the shelve
		// "Created shelve sh:12@UE5PlasticPluginDev@test@cloud (mount:'/')"
		FString CreatedShelveStatus = Results[Results.Num() - 1];
		if (CreatedShelveStatus.StartsWith(TEXT("Created shelve sh:")))
		{
			CreatedShelveStatus.RightChopInline(18);
			int32 SeparatorIndex;
			if (CreatedShelveStatus.FindChar(TEXT('@'), SeparatorIndex))
			{
				CreatedShelveStatus.LeftInline(SeparatorIndex);
				OutShelveId = FCString::Atoi(*CreatedShelveStatus);
			}
		}
	}

	return OutShelveId != ISourceControlState::INVALID_REVISION;
}

static bool DeleteShelve(const int32 InShelveId, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Results;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("delete"));
	Parameters.Add(FString::Printf(TEXT("sh:%d"), InShelveId));
	return PlasticSourceControlUtils::RunCommand(TEXT("shelveset"), Parameters, TArray<FString>(), Results, OutErrorMessages);
}

FName FPlasticShelveWorker::GetName() const
{
	return "Shelve";
}

bool FPlasticShelveWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticShelveWorker::Execute);

	check(InCommand.Operation->GetName() == GetName());
	TSharedRef<FShelve, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FShelve>(InCommand.Operation);

	FPlasticSourceControlChangelist Changelist(InCommand.Changelist);

	TArray<FString> FilesToShelve = InCommand.Files;

	int32 PreviousShelveId = ISourceControlState::INVALID_REVISION;

	InCommand.bCommandSuccessful = true;

	if (InCommand.Changelist.IsInitialized())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);

		PreviousShelveId = ChangelistState->ShelveId;

		// If the command has specified a changelist but no files, then get all files from it
		if (FilesToShelve.Num() == 0)
		{
			FilesToShelve = FileNamesFromFileStates(ChangelistState->Files);
		}
		// If the command has specified some files, and if there was already a shelve,
		// ensure that the previous shelved files are also put back into the new one.
		// This is to mimic Perforce behavior, where we can add files to a shelve after we made more changes.
		// NOTE: this is a workaround for the fact that we cannot edit an existing shelve.
		else if (ChangelistState->ShelvedFiles.Num() > 0)
		{
			for (const FSourceControlStateRef& ShelveFile : ChangelistState->ShelvedFiles)
			{
				FilesToShelve.AddUnique(ShelveFile->GetFilename());
			}
		}

		// Ensure that all the files to shelve are indeed in the changelist, so that we can actually shelve them
		// NOTE: this is because of the workaround that requires to create a new shelve and delete the old one
		for (FString& FileToShelve : FilesToShelve)
		{
			if (!ChangelistState->Files.ContainsByPredicate([&FileToShelve](const FSourceControlStateRef& FileState) { return FileToShelve == FileState->GetFilename(); }))
			{
				FPaths::MakePathRelativeTo(FileToShelve, *FPaths::ProjectDir());
				UE_LOGF(LogSourceControl, Error, "The file /%ls is not in the changelist anymore, so the shelve cannot be updated. Unshelve the corresponding change and retry.", *FileToShelve);
				InCommand.bCommandSuccessful = false;
			}
		}
	}

	if (InCommand.bCommandSuccessful)
	{
		// If the command is issued on the default changelist, then we need to create a new changelist,
		// move the files to the new changelist, then shelve the files
		if (InCommand.Changelist.IsDefault())
		{
			// Create a new numbered persistent changelist ala Perforce
			Changelist = CreatePendingChangelist(GetProvider(), Operation->GetDescription().ToString(), InCommand.InfoMessages, InCommand.ErrorMessages);
			if (Changelist.IsInitialized())
			{
				InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), Changelist, FilesToShelve, InCommand.InfoMessages, InCommand.ErrorMessages);
				if (InCommand.bCommandSuccessful)
				{
					MovedFiles = FilesToShelve;
				}
			}
		}
	}

	if (InCommand.bCommandSuccessful)
	{
		// Remove unmodified files from the list of files to shelve
		int32 i = 0;
		while (i < FilesToShelve.Num())
		{
			FString& FileToShelve = FilesToShelve[i];
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(FileToShelve);
			if (FileState->IsModified())
			{
				i++;
			}
			else
			{
				FPaths::MakePathRelativeTo(FileToShelve, *FPaths::ProjectDir());
				UE_LOGF(LogSourceControl, Warning, "The file /%ls is unchanged, it cannot be shelved.", *FileToShelve);
				FilesToShelve.RemoveAt(i);
			}
		}

		ChangelistDescription = *Operation->GetDescription().ToString();

		if (FilesToShelve.Num() > 0)
		{
			InCommand.bCommandSuccessful = CreateShelve(Changelist.GetName(), ChangelistDescription, FilesToShelve, ShelveId, InCommand.ErrorMessages);
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "No file to Shelve");
			InCommand.bCommandSuccessful = false;
		}
		if (InCommand.bCommandSuccessful)
		{
			InChangelistToUpdate = InCommand.Changelist;
			OutChangelistToUpdate = Changelist;
			ShelvedFiles = FilesToShelve;

			// If there was already a shelve, we have now created a new one with updated files, so we must delete the old one
			if (PreviousShelveId != ISourceControlState::INVALID_REVISION)
			{
				DeleteShelve(PreviousShelveId, InCommand.ErrorMessages);
			}
		}
		else
		{
			// In case of failure to shelve, if we had to create a new changelist, move the files back to the default changelist and delete the changelist
			if (Changelist != InCommand.Changelist)
			{
				if (MovedFiles.Num() > 0)
				{
					MoveFilesToChangelist(GetProvider(), InCommand.Changelist, MovedFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
				}

				DeleteChangelist(GetProvider(), Changelist, InCommand.InfoMessages, InCommand.ErrorMessages);
				GetProvider().RemoveChangelistFromCache(Changelist);
			}
		}
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticShelveWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticShelveWorker::UpdateStates);

	if (OutChangelistToUpdate.IsInitialized())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> DestinationChangelistState = GetProvider().GetStateInternal(OutChangelistToUpdate);

		bool bMovedFiles = false;

		// If we moved files to a new changelist, then we must make sure that the files are properly moved
		if (InChangelistToUpdate != OutChangelistToUpdate && MovedFiles.Num() > 0)
		{
			const FDateTime Now = FDateTime::Now();
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> SourceChangelistState = GetProvider().GetStateInternal(InChangelistToUpdate);

			DestinationChangelistState->Changelist = OutChangelistToUpdate;
			DestinationChangelistState->Description = ChangelistDescription;

			for (const FString& MovedFile : MovedFiles)
			{
				TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(MovedFile);

				SourceChangelistState->Files.Remove(FileState);
				DestinationChangelistState->Files.Add(FileState);
				FileState->Changelist = OutChangelistToUpdate;
				FileState->TimeStamp = Now;
			}

			bMovedFiles = true;
		}

		DestinationChangelistState->ShelveId = ShelveId;
		DestinationChangelistState->ShelveDate = FDateTime::Now();

		// And finally, add the shelved files to the changelist state
		DestinationChangelistState->ShelvedFiles.Reset(ShelvedFiles.Num());
		for (FString ShelvedFile : ShelvedFiles)
		{
			TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FileState = GetProvider().GetStateInternal(ShelvedFile);
			PlasticSourceControlUtils::AddShelvedFileToChangelist(DestinationChangelistState.Get(), MoveTemp(ShelvedFile), FileState->WorkspaceState, FString(FileState->MovedFrom));
		}

		return bMovedFiles || ShelvedFiles.Num() > 0;
	}
	else
	{
		return false;
	}
}



FName FPlasticUnshelveWorker::GetName() const
{
	return "Unshelve";
}

bool FPlasticUnshelveWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticUnshelveWorker::Execute);

	// Get the state of the changelist to operate on
	TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);

	InCommand.bCommandSuccessful = (ChangelistState->ShelveId != ISourceControlState::INVALID_REVISION);

	// Detect if any file to unshelve has some local modification, which would fail the "unshelve" operation with a merge conflict
	// NOTE: we could decide to automatically undo the local changes in order for this process to be automatic, like with Perforce
	for (FString& File : InCommand.Files)
	{
		if (ChangelistState->Files.FindByPredicate([&File](const FSourceControlStateRef& FileState) { return FileState->GetFilename().Equals(File, ESearchCase::IgnoreCase); }))
		{
			FPaths::MakePathRelativeTo(File, *FPaths::ProjectDir());
			UE_LOGF(LogSourceControl, Error, "Revert /%ls before unshelving the corresponding change from the shelve.", *File);
			InCommand.bCommandSuccessful = false;
		}
	}

	if (InCommand.bCommandSuccessful)
	{
		// Get the list of files to unshelve if not all of them are selected
		TArray<FString> FilesToUnshelve;
		if (InCommand.Files.Num() < ChangelistState->ShelvedFiles.Num())
		{
			if (GetProvider().GetPlasticScmVersion() < PlasticSourceControlVersions::ShelvesetApplySelection)
			{
				// On old version, don't unshelve the files if they are not all selected (since we couldn't apply only a selection of files from a shelve)
				UE_LOGF(LogSourceControl, Error,
					"Unity Version Control %ls cannot unshelve a selection of files from a shelve. Unshelve them all at once or update to %ls or above.",
					*GetProvider().GetPlasticScmVersion().String,
					*PlasticSourceControlVersions::ShelvesetApplySelection.String
				);
				return false;
			}
			const FString PathToWorkspaceRoot = GetProvider().GetPathToWorkspaceRoot();
			FilesToUnshelve.Reset(InCommand.Files.Num());
			for (FString File : InCommand.Files)
			{
				// Make path relative the workspace root, since the shelveset apply operation require server paths
				FPaths::MakePathRelativeTo(File, *PathToWorkspaceRoot);
				FilesToUnshelve.Add(TEXT("/") + File);
			}
		}

		// Make the Editor Unlink the assets if they are loaded in memory so that source control can override the corresponding files
		PackageUtils::UnlinkPackagesInMainThread(InCommand.Files);

		{
			// 'cm shelveset apply sh:88 "/Content/Blueprints/BP_CheckedOut.uasset"'
			TArray<FString> Parameters;
			Parameters.Add(TEXT("apply"));
			Parameters.Add(FString::Printf(TEXT("sh:%d"), ChangelistState->ShelveId));
			// TODO --noinput
			InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("shelveset"), Parameters, FilesToUnshelve, InCommand.InfoMessages, InCommand.ErrorMessages);
		}

		// Reload packages that where updated by the Unshelve operation (and the current map if needed)
		PackageUtils::ReloadPackagesInMainThread(InCommand.Files);
	}

	if (InCommand.bCommandSuccessful)
	{
		// move all the unshelved files back to the changelist
		InCommand.bCommandSuccessful = MoveFilesToChangelist(GetProvider(), InCommand.Changelist, InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	if (InCommand.bCommandSuccessful)
	{
		// now update the status of our files
		PlasticSourceControlUtils::InvalidateLocksCache();
		PlasticSourceControlUtils::RunUpdateStatus(InCommand.Files, PlasticSourceControlUtils::EStatusSearchType::ControlledOnly, false, InCommand.ErrorMessages, States, InCommand.ChangesetNumber);

		ChangelistToUpdate = InCommand.Changelist;
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticUnshelveWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticUnshelveWorker::UpdateStates);

	UpdateChangelistState(GetProvider(), ChangelistToUpdate, States);

	return PlasticSourceControlUtils::UpdateCachedStates(MoveTemp(States));
}

FName FPlasticDeleteShelveWorker::GetName() const
{
	return "DeleteShelved";
}

bool FPlasticDeleteShelveWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteShelveWorker::Execute);

	TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(InCommand.Changelist);

	InCommand.bCommandSuccessful = (ChangelistState->ShelveId != ISourceControlState::INVALID_REVISION);

	// Get the list of files to keep in the shelve if not all of them are selected (to create a new shelve with them)
	TArray<FString> FilesToShelve;
	if (InCommand.Files.Num() < ChangelistState->ShelvedFiles.Num())
	{
		for (const auto& ShelveState : ChangelistState->ShelvedFiles)
		{
			if (!InCommand.Files.Contains(ShelveState->GetFilename()))
			{
				FString File = ShelveState->GetFilename();

				// Check that all this files are is still in the corresponding changelist, else we won't be able to create the new shelve!
				if (ChangelistState->Files.ContainsByPredicate([&File](FSourceControlStateRef& State) { return File == State->GetFilename(); }))
				{
					FilesToShelve.Add(MoveTemp(File));
				}
				else
				{
					FPaths::MakePathRelativeTo(File, *FPaths::ProjectDir());
					UE_LOGF(LogSourceControl, Error, "The file /%ls is not in the changelist anymore, so the shelve cannot be updated. Unshelve the corresponding change and retry.", *File);
					InCommand.bCommandSuccessful = false;
				}
			}
		}
	}

	if (InCommand.bCommandSuccessful)
	{
		ChangelistToUpdate = InCommand.Changelist;
		FilesToRemove = InCommand.Files;
	}

	if (InCommand.bCommandSuccessful && FilesToShelve.Num() > 0)
	{
		// Create a new shelve with the other files
		InCommand.bCommandSuccessful = CreateShelve(InCommand.Changelist.GetName(), ChangelistState->GetDescriptionText().ToString(), FilesToShelve, ShelveId, InCommand.ErrorMessages);
	}

	if (InCommand.bCommandSuccessful)
	{
		// Delete the old shelve
		InCommand.bCommandSuccessful = DeleteShelve(ChangelistState->ShelveId, InCommand.ErrorMessages);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticDeleteShelveWorker::UpdateStates()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticDeleteShelveWorker::UpdateStates);

	if (ChangelistToUpdate.IsInitialized())
	{
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = GetProvider().GetStateInternal(ChangelistToUpdate);

		ChangelistState->ShelveId = ShelveId;

		if (FilesToRemove.Num() > 0)
		{
			// NOTE: for now, Unity Version Control cannot delete a selection of files from a shelve, so FilesToRemove and this specific case aren't really needed (yet)
			return ChangelistState->ShelvedFiles.RemoveAll([this](FSourceControlStateRef& State) -> bool
				{
					return Algo::AnyOf(FilesToRemove, [&State](const FString& File) {
						return State->GetFilename() == File;
					});
				}) > 0;
		}
		else
		{
			bool bHadShelvedFiles = (ChangelistState->ShelvedFiles.Num() > 0);
			ChangelistState->ShelvedFiles.Reset();
			return bHadShelvedFiles;
		}
	}
	else
	{
		return false;
	}
}

// Copied from SSourceControlReview.cpp of the ChangelistReview Editor plugin
namespace ReviewHelpers
{
	const FString FileDepotKey = TEXT("depotFile");
	const FString FileRevisionKey = TEXT("rev");
	const FString FileActionKey = TEXT("action");
	const FString TimeKey = TEXT("time");
	const FString AuthorKey = TEXT("user");
	const FString DescriptionKey = TEXT("desc");
	const FString ChangelistStatusKey = TEXT("status");
	const FString ChangelistPendingStatusKey = TEXT("pending");
	constexpr int32 RecordIndex = 0;
} // namespace ReviewHelpers

FName FPlasticGetChangelistDetailsWorker::GetName() const
{
	return "GetChangelistDetails";
}

bool FPlasticGetChangelistDetailsWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetChangelistDetailsWorker::Execute);

	TSharedRef<FGetChangelistDetails, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetChangelistDetails>(InCommand.Operation);

	// Note: Changelists are local construct so we have to interpret this as a Shelve Id instead
	const FString& ShelveId = Operation->GetChangelistNumber();
	if (ShelveId.IsEmpty())
	{
		InCommand.bCommandSuccessful = false;
		InCommand.ErrorMessages.Add((LOCTEXT("GetChangelistDetailsEmptyId", "GetChangelistDetails failed. Shelve Id is empty.").ToString()));
		return false;
	}

	FString Comment;
	FString Owner;
	FDateTime Date;
	TArray<FPlasticSourceControlRevision> BaseRevisions;
	InCommand.bCommandSuccessful = PlasticSourceControlUtils::RunGetShelve(FCString::Atoi(*ShelveId), Comment, Date, Owner, BaseRevisions, InCommand.ErrorMessages);
	if (!InCommand.bCommandSuccessful)
	{
		InCommand.bCommandSuccessful = false;
		InCommand.ErrorMessages.Add((LOCTEXT("GetChangelistDetailsInvalidId", "GetChangelistDetails failed. Shelve Id is invalid.").ToString()));
		return false;
	}

	UE_LOGF(LogSourceControl, Log, "GetChangelistDetails: %d files in shelve %ls", BaseRevisions.Num(), *ShelveId);

	TMap<FString, FString> Record;

	Record.Add({ ReviewHelpers::ChangelistStatusKey, ReviewHelpers::ChangelistPendingStatusKey });
	Record.Add({ ReviewHelpers::AuthorKey, Owner });
	Record.Add({ ReviewHelpers::DescriptionKey, Comment });
	Record.Add({ ReviewHelpers::TimeKey, LexToString(Date.ToUnixTimestamp()) });

	uint32  RecordFileIndex = 0;
	for (auto& Revision : BaseRevisions)
	{
		// String representation of the current file index
		FString RecordFileIndexStr = LexToString(RecordFileIndex);
		// The p4 records is the map a file key starts with "depotFile" and is followed by file index
		FString RecordFileMapKey = ReviewHelpers::FileDepotKey + RecordFileIndexStr;
		// The p4 records is the map a revision key starts with "rev" and is followed by file index
		FString RecordRevisionMapKey = ReviewHelpers::FileRevisionKey + RecordFileIndexStr;
		// The p4 records is the map a revision key starts with "action" and is followed by file index
		FString RecordActionMapKey = ReviewHelpers::FileActionKey + RecordFileIndexStr;

		UE_LOGF(LogSourceControl, Log, "GetChangelistDetails: %ls baserevid:%d %ls", *Revision.Filename, Revision.RevisionId, *Revision.Action);

		Record.Add({ MoveTemp(RecordFileMapKey), MoveTemp(Revision.Filename) });
		Record.Add({ MoveTemp(RecordRevisionMapKey), LexToString(Revision.RevisionId) });
		Record.Add({ MoveTemp(RecordActionMapKey), Revision.Action });

		RecordFileIndex++;
	}

	TArray<TMap<FString, FString>> ChangelistRecord;
	ChangelistRecord.Add(MoveTemp(Record));
	Operation->SetChangelistDetails(MoveTemp(ChangelistRecord));

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetChangelistDetailsWorker::UpdateStates()
{
	return false;
}

FName FPlasticGetFileWorker::GetName() const
{
	return "GetFile";
}

bool FPlasticGetFileWorker::Execute(FPlasticSourceControlCommand& InCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticGetFileWorker::Execute);

	TSharedRef<FGetFile, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FGetFile>(InCommand.Operation);

	FPlasticSourceControlRevision SourceControlRevision;
	SourceControlRevision.Filename = FPaths::ConvertRelativePathToFull(Operation->GetDepotFilePath());

	if (Operation->IsShelve())
	{
		SourceControlRevision.ShelveId = FCString::Atoi(*Operation->GetChangelistNumber());
		UE_LOGF(LogSourceControl, Log, "GetFile(ShelveId:%d)", SourceControlRevision.ShelveId);
	}
	else
	{
		SourceControlRevision.RevisionId = FCString::Atoi(*Operation->GetRevisionNumber());
		UE_LOGF(LogSourceControl, Log, "GetFile(revid:%d)", SourceControlRevision.RevisionId);
	}

	FString OutFilename;
	InCommand.bCommandSuccessful = SourceControlRevision.Get(OutFilename, InCommand.Concurrency);
	if (InCommand.bCommandSuccessful)
	{
		Operation->SetOutPackageFilename(OutFilename);
	}

	return InCommand.bCommandSuccessful;
}

bool FPlasticGetFileWorker::UpdateStates()
{
	return false;
}

#undef LOCTEXT_NAMESPACE
