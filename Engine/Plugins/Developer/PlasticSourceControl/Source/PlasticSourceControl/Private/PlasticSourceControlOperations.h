// Copyright Epic Games, Inc. All Rights Reserved.

// Specialization of classes defined in Engine\Source\Developer\SourceControl\Public\SourceControlOperations.h

#pragma once

#include "CoreMinimal.h"
#include "IPlasticSourceControlWorker.h"
#include "PlasticSourceControlRevision.h"
#include "PlasticSourceControlState.h"
#include "SourceControlOperationBase.h"
#include "SourceControlOperations.h"

#include "PlasticSourceControlChangelist.h"
#include "PlasticSourceControlChangelistState.h"

class FPlasticSourceControlProvider;
typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;
typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedPtr<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetPtr;
typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;


/**
 * Internal operation used to revert checked-out unchanged files
 *
 * NOTE: added to Engine in Unreal Engine 5 for changelists
*/
class FPlasticRevertUnchanged final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;
};


/**
 * Internal operation used to sync all files in the workspace
 *
 * NOTE: override the standard "Sync" operation to provide a list of updated files and a custom progress string
*/
class FPlasticSyncAll final : public FSync
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	/** List of files updated by the operation */
	TArray<FString> UpdatedFiles;
};


/**
 * Internal operation used to revert checked-out files
*/
class FPlasticRevertAll final : public FRevert
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	/** List of files updated by the operation */
	TArray<FString> UpdatedFiles;
};


/**
 * Internal operation used to revert files to a previous revision
*/
class FPlasticRevertToRevision final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	/** List of files updated by the operation */
	TArray<FString> UpdatedFiles;

	// Changeset to revert the files to
	int32 ChangesetId = ISourceControlState::INVALID_REVISION;
};


/**
* Internal operation used to create a new Workspace and a new Repository
*/
class FPlasticMakeWorkspace final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	FString WorkspaceName;
	FString RepositoryName;
	FString ServerUrl;
	bool bPartialWorkspace = false;
};


/**
 * Internal operation used to switch to a partial workspace
*/
class FPlasticSwitchToPartialWorkspace final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;
};


/**
 * Internal operation to list locks, aka "cm lock list"
*/
class FPlasticGetLocks final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// List of locks found
	TArray<FPlasticSourceControlLockRef> Locks;
};


/**
 * Internal operation used to release or remove Lock(s) on file(s)
*/
class FPlasticUnlock final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Locks to unlock, including the Item Id and branch name
	TArray<FPlasticSourceControlLockRef> Locks;

	// Release the Lock(s), and optionally remove (delete) them completely
	bool bRemove = false;
};


/**
 * Internal operation to list branches, aka "cm find branch"
*/
class FPlasticGetBranches final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Limit the list of branches to ones created from this date (optional, filtering enabled by default)
	FDateTime FromDate;

	// List of branches found
	TArray<FPlasticSourceControlBranchRef> Branches;
};


/**
 * Internal operation used to switch the workspace to a branch or a changeset
*/
class FPlasticSwitch final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Branch to switch the workspace to (optional, only apply if ChangesetId is not set)
	FString BranchName;

	// Changeset to switch the workspace to (optional, overrides the BranchName if set)
	int32 ChangesetId = ISourceControlState::INVALID_REVISION;

	/** List of files updated by the operation */
	TArray<FString> UpdatedFiles;
};


/**
 * Internal operation used to merge a branch into the current branch
*/
class FPlasticMergeBranch final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Branch to switch the workspace to
	FString BranchName;

	/** List of files updated by the operation */
	TArray<FString> UpdatedFiles;
};


/**
 * Internal operation used to create a branch
*/
class FPlasticCreateBranch final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	FString BranchName;
	FString Comment;
};


/**
 * Internal operation used to rename a branch
*/
class FPlasticRenameBranch final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	FString OldName;
	FString NewName;
};


/**
 * Internal operation used to deletes branches
*/
class FPlasticDeleteBranches final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	TArray<FString> BranchNames;
};


/**
 * Internal operation to list changesets, aka "cm find changesets"
*/
class FPlasticGetChangesets final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Limit the list of changesets to ones created from this date (optional, filtering enabled by default)
	FDateTime FromDate;

	// List of changesets found
	TArray<FPlasticSourceControlChangesetRef> Changesets;
};


/**
 * Internal operation to list files in a changeset, using "cm log cs:<ChangesetId>"
*/
class FPlasticGetChangesetFiles final : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override;

	virtual FText GetInProgressString() const override;

	// Changeset to list files from
	FPlasticSourceControlChangesetPtr Changeset;

	// List of files changed in the changeset
	TArray<FPlasticSourceControlStateRef> Files;
};


/** Called when first activated on a project, and then at project load time.
 *  Look for the root directory of the Plastic workspace (where the ".plastic/" subdirectory is located). */
class FPlasticConnectWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticConnectWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticConnectWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

class FPlasticCheckOutWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticCheckOutWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
		, InChangelist(FPlasticSourceControlChangelist::DefaultChangelist) // By default, add checked out files in the default changelist.
	{}
	virtual ~FPlasticCheckOutWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;

	/** Changelist we checked-out files to (defaults to the Default changelist) */
	FPlasticSourceControlChangelist InChangelist;
};

/** Check-in a set of file to the local depot. */
class FPlasticCheckInWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticCheckInWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticCheckInWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;

	/** Changelist we submitted */
	FPlasticSourceControlChangelist InChangelist;
};

/** Add an untracked file to source control (so only a subset of the Plastic add command). */
class FPlasticMarkForAddWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticMarkForAddWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
		, InChangelist(FPlasticSourceControlChangelist::DefaultChangelist) // By default, add new files in the default changelist.
	{}
	virtual ~FPlasticMarkForAddWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;

	/** Changelist we added files to (defaults to the Default changelist) */
	FPlasticSourceControlChangelist InChangelist;
};

/** Delete a file and remove it from source control. */
class FPlasticDeleteWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticDeleteWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
		, InChangelist(FPlasticSourceControlChangelist::DefaultChangelist) // By default, add deleted files in the default changelist.
	{}
	virtual ~FPlasticDeleteWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;

	/** Changelist we delete files to (defaults to the Default changelist) */
	FPlasticSourceControlChangelist InChangelist;
};

/** Revert any change to a file to its state on the local depot. */
class FPlasticRevertWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticRevertWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticRevertWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Revert only unchanged file(s) (uncheckout). */
class FPlasticRevertUnchangedWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticRevertUnchangedWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticRevertUnchangedWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Revert all checked-out file(s). */
class FPlasticRevertAllWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticRevertAllWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticRevertAllWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Revert file(s) to selected revision. */
class FPlasticRevertToRevisionWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticRevertToRevisionWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticRevertToRevisionWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Create a new Workspace and a new Repository */
class FPlasticMakeWorkspaceWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticMakeWorkspaceWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticMakeWorkspaceWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** Switch to Partial Workspace. */
class FPlasticSwitchToPartialWorkspaceWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticSwitchToPartialWorkspaceWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticSwitchToPartialWorkspaceWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** list locks. */
class FPlasticGetLocksWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticGetLocksWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetLocksWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** release or remove Lock(s) on file(s). */
class FPlasticUnlockWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticUnlockWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticUnlockWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** list branches. */
class FPlasticGetBranchesWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticGetBranchesWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetBranchesWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** Switch workspace to another branch. */
class FPlasticSwitchWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticSwitchWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticSwitchWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Merge a branch to the current branch. */
class FPlasticMergeBranchWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticMergeBranchWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticMergeBranchWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Create a new child branch. */
class FPlasticCreateBranchWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticCreateBranchWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticCreateBranchWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** Rename a branch. */
class FPlasticRenameBranchWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticRenameBranchWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticRenameBranchWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** Delete branches. */
class FPlasticDeleteBranchesWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticDeleteBranchesWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticDeleteBranchesWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** list changesets. */
class FPlasticGetChangesetsWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticGetChangesetsWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetChangesetsWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

	// Current changeset the workspace is on (at the end of the operation)
	int32 CurrentChangesetId;
};

/** list files in changeset. */
class FPlasticGetChangesetFilesWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticGetChangesetFilesWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetChangesetFilesWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

/** Plastic update the workspace to latest changes */
class FPlasticSyncWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticSyncWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticSyncWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Get source control status of files on local workspace. */
class FPlasticUpdateStatusWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticUpdateStatusWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticUpdateStatusWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Copy or Move operation on a single file */
class FPlasticCopyWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticCopyWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticCopyWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

/** Plastic command to mark the conflict as solved */
class FPlasticResolveWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticResolveWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticResolveWorker() = default;
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

private:
	/** Temporary states for results */
	TArray<FPlasticSourceControlState> States;
};

class FPlasticGetPendingChangelistsWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticGetPendingChangelistsWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetPendingChangelistsWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** Temporary states for results */
	TArray<FPlasticSourceControlChangelistState> OutChangelistsStates;
	TArray<TArray<FPlasticSourceControlState>> OutCLFilesStates;

private:
	/** Controls whether or not we will remove changelists from the cache after a full update */
	bool bCleanupCache = false;
};

class FPlasticNewChangelistWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticNewChangelistWorker(FPlasticSourceControlProvider& InSourceControlProvider);
	virtual ~FPlasticNewChangelistWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	/** New changelist information */
	FPlasticSourceControlChangelist NewChangelist;
	FPlasticSourceControlChangelistState NewChangelistState;

	/** Files that were moved */
	TArray<FString> MovedFiles;
};

class FPlasticDeleteChangelistWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticDeleteChangelistWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticDeleteChangelistWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	FPlasticSourceControlChangelist DeletedChangelist;
};

class FPlasticEditChangelistWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticEditChangelistWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticEditChangelistWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

public:
	FPlasticSourceControlChangelist EditedChangelist;
	FString EditedDescription;

	/** Reopened files (moved to a new changelist, if any, when editing the Default changelist) */
	TArray<FString> ReopenedFiles;
};

class FPlasticReopenWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticReopenWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticReopenWorker() = default;
	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

protected:
	/** Reopened files (moved to a new changelist) */
	TArray<FString> ReopenedFiles;

	/** Destination changelist */
	FPlasticSourceControlChangelist DestinationChangelist;
};

class FPlasticShelveWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticShelveWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticShelveWorker() = default;

	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

protected:
	int32 ShelveId = ISourceControlState::INVALID_REVISION;

	TArray<FString> ShelvedFiles;

	/** Files that were moved to a new changelist if shelving from the Default Changelist */
	TArray<FString> MovedFiles;

	/** Changelist description if needed */
	FString ChangelistDescription;

	/** Changelist(s) to be updated */
	FPlasticSourceControlChangelist InChangelistToUpdate;
	FPlasticSourceControlChangelist OutChangelistToUpdate;
};

class FPlasticUnshelveWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticUnshelveWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticUnshelveWorker() = default;

	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

protected:
	/** List of files states after the unshelve */
	TArray<FPlasticSourceControlState> States;

	/** Changelist to be updated */
	FPlasticSourceControlChangelist ChangelistToUpdate;
};

class FPlasticDeleteShelveWorker final : public IPlasticSourceControlWorker
{
public:
	explicit FPlasticDeleteShelveWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticDeleteShelveWorker() = default;

	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;

protected:
	/** List of files to remove from shelved files in changelist state */
	TArray<FString> FilesToRemove;

	/** Changelist to be updated */
	FPlasticSourceControlChangelist ChangelistToUpdate;

	/** Id of the new shelve (if only a selection of files are deleted from the shelve) */
	int32 ShelveId = ISourceControlState::INVALID_REVISION;
};

class FPlasticGetChangelistDetailsWorker final : public IPlasticSourceControlWorker
{
public:
	FPlasticGetChangelistDetailsWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetChangelistDetailsWorker() = default;

	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};

class FPlasticGetFileWorker final : public IPlasticSourceControlWorker
{
public:
	FPlasticGetFileWorker(FPlasticSourceControlProvider& InSourceControlProvider)
		: IPlasticSourceControlWorker(InSourceControlProvider)
	{}
	virtual ~FPlasticGetFileWorker() = default;

	// IPlasticSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(FPlasticSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() override;
};
