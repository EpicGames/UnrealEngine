// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlParsers.h"

#include "PlasticSourceControlBranch.h"
#include "PlasticSourceControlChangeset.h"
#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlState.h"
#include "PlasticSourceControlUtils.h"
#include "PlasticSourceControlVersions.h"
#include "ISourceControlModule.h"

#include "HAL/PlatformFile.h"
#include "Misc/Paths.h"
#include "XmlParser.h"

#include "PlasticSourceControlChangelist.h"
#include "PlasticSourceControlChangelistState.h"

namespace PlasticSourceControlParsers
{

#define FILE_STATUS_SEPARATOR TEXT(";")


/**
 * Parse the output of the "cm profile list --format="{server};{user}" command
 *
 * Example:
localhost:8087|sebastien.rombauts
local|sebastien.rombauts@unity3d.com
SRombautsU@cloud|sebastien.rombauts@unity3d.com
*/
bool ParseProfileInfo(TArray<FString>& InResults, const FString& InServerUrl, FString& OutUserName)
{
	for (const FString& Result : InResults)
	{
		TArray<FString> ProfileInfos;
		Result.ParseIntoArray(ProfileInfos, FILE_STATUS_SEPARATOR, false); // Don't cull empty values
		if (ProfileInfos.Num() == 2)
		{
			if (ProfileInfos[0] == InServerUrl)
			{
				OutUserName = ProfileInfos[1];
				return true;
			}
		}
	}

	return false;
}

/**
 * Parse  workspace information, in the form "Branch /main@UE5PlasticPluginDev@localhost:8087"
 *                                        or "Branch /main@UE5PlasticPluginDev@test@cloud" (when connected to the cloud)
 *                                        or "Branch /main@rep:UE5OpenWorldPerfTest@repserver:test@cloud"
 *                                        or "Changeset 1234@UE5PlasticPluginDev@test@cloud" (when the workspace is switched to a changeset instead of a branch)
 *                                        or "Label 1.10.0@UE5PlasticPluginDev@test@cloud" (when the workspace is switched to a label instead of a branch)
*/
bool ParseWorkspaceInfo(TArray<FString>& InResults, FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl)
{
	if (InResults.Num() == 0)
	{
		return false;
	}

	static const FString BranchPrefix(TEXT("Branch "));
	static const FString ChangesetPrefix(TEXT("Changeset "));
	static const FString LabelPrefix(TEXT("Label "));
	static const FString RepPrefix(TEXT("rep:"));
	static const FString RepserverPrefix(TEXT("repserver:"));
	bool bIsABranch = false;
	FString& WorkspaceInfo = InResults[0];
	if (WorkspaceInfo.StartsWith(BranchPrefix, ESearchCase::CaseSensitive))
	{
		bIsABranch = true;
		WorkspaceInfo.RightChopInline(BranchPrefix.Len());
	}
	else if (WorkspaceInfo.StartsWith(ChangesetPrefix, ESearchCase::CaseSensitive))
	{
		WorkspaceInfo.RightChopInline(ChangesetPrefix.Len());
	}
	else if (WorkspaceInfo.StartsWith(LabelPrefix, ESearchCase::CaseSensitive))
	{
		WorkspaceInfo.RightChopInline(LabelPrefix.Len());
	}
	else
	{
		return false;
	}

	TArray<FString> WorkspaceInfos;
	WorkspaceInfo.ParseIntoArray(WorkspaceInfos, TEXT("@"), false); // Don't cull empty values
	if (WorkspaceInfos.Num() >= 3)
	{
		OutWorkspaceSelector = MoveTemp(WorkspaceInfos[0]);
		if (bIsABranch)
		{
			OutBranchName = OutWorkspaceSelector;
		}
		OutRepositoryName = MoveTemp(WorkspaceInfos[1]);
		OutServerUrl = MoveTemp(WorkspaceInfos[2]);

		if (OutRepositoryName.StartsWith(RepPrefix, ESearchCase::CaseSensitive))
		{
			OutRepositoryName.RightChopInline(RepPrefix.Len());
		}

		if (OutServerUrl.StartsWith(RepserverPrefix, ESearchCase::CaseSensitive))
		{
			OutServerUrl.RightChopInline(RepserverPrefix.Len());
		}

		if (WorkspaceInfos.Num() > 3) // (when connected to the cloud)
		{
			OutServerUrl.Append(TEXT("@"));
			OutServerUrl.Append(MoveTemp(WorkspaceInfos[3]));
		}
	}
	else
	{
		return false;
	}

	return true;
}

/**
* Parse the current changeset from the header returned by "cm status --machinereadable --fieldseparator=;"
*
* Get workspace status in one of the form
STATUS;41;UEPlasticPluginDev;localhost:8087
STATUS;41;UEPlasticPluginDev;test@cloud
*
* @note The semicolon (";") that is used as filedseparator can also be used in the name of a repository.
*       This wouldn't be an issue with the current code, but we have to keep that in mind for future evolutions.
*/
bool GetChangesetFromWorkspaceStatus(const TArray<FString>& InResults, int32& OutChangeset)
{
	if (InResults.Num() > 0)
	{
		const FString& WorkspaceStatus = InResults[0];
		TArray<FString> WorkspaceInfos;
		WorkspaceStatus.ParseIntoArray(WorkspaceInfos, FILE_STATUS_SEPARATOR, false); // Don't cull empty values in csv
		if (WorkspaceInfos.Num() >= 4)
		{
			OutChangeset = FCString::Atoi(*WorkspaceInfos[1]);
			return true;
		}
	}

	return false;
}

/**
 * Interpret the 2-to-8 letters file status from the given cm "status" result.
 *
 * @param InFileStatus The 2-to-8 letters file status from the given cm "status" result
 * @param bInUsesCheckedOutChanged If using the new --iscochanged "CO+CH"
 * @return EWorkspaceState
 *
 * @see #ParseFileStatusResult() for examples of results from "cm status --machinereadable"
*/
static EWorkspaceState StateFromStatus(const FString& InFileStatus, const bool bInUsesCheckedOutChanged)
{
	EWorkspaceState State;

	if (InFileStatus == TEXT("CH")) // Modified but not Checked-Out
	{
		State = EWorkspaceState::Changed;
	}
	else if (InFileStatus == TEXT("CO")) // Checked-Out with no change, or "don't know" if using on an old version of cm
	{
		// Recent version can distinguish between CheckedOut with or with no changes
		if (bInUsesCheckedOutChanged)
		{
			State = EWorkspaceState::CheckedOutUnchanged; // Recent version; here it's checkedout with no change
		}
		else
		{
			State = EWorkspaceState::CheckedOutChanged; // Older version; need to assume it is changed to retain behavior
		}
	}
	else if (InFileStatus == TEXT("CO+CH")) // Checked-Out and changed from the new --iscochanged
	{
		State = EWorkspaceState::CheckedOutChanged; // Recent version; here it's checkedout with changes
	}
	else if (InFileStatus.Contains(TEXT("CP"))) // "CP", "CO+CP"
	{
		State = EWorkspaceState::Copied;
	}
	else if (InFileStatus.Contains(TEXT("MV"))) // "MV", "CO+MV", "CO+CH+MV", "CO+RP+MV"
	{
		State = EWorkspaceState::Moved; // Moved/Renamed
	}
	else if (InFileStatus.Contains(TEXT("RP"))) // "RP", "CO+RP", "CO+RP+CH", "CO+CH+RP"
	{
		State = EWorkspaceState::Replaced;
	}
	else if (InFileStatus == TEXT("AD"))
	{
		State = EWorkspaceState::Added;
	}
	else if ((InFileStatus == TEXT("PR")) || (InFileStatus == TEXT("LM"))) // Not Controlled/Not in Depot/Untracked (or Locally Moved/Renamed)
	{
		State = EWorkspaceState::Private;
	}
	else if (InFileStatus == TEXT("IG"))
	{
		State = EWorkspaceState::Ignored;
	}
	else if (InFileStatus == TEXT("DE"))
	{
		State = EWorkspaceState::Deleted; // Deleted (removed from source control)
	}
	else if (InFileStatus.Contains(TEXT("LD"))) // "LD", "AD+LD"
	{
		State = EWorkspaceState::LocallyDeleted; // Locally Deleted (ie. missing)
	}
	else
	{
		UE_LOGF(LogSourceControl, Warning, "Unknown file status '%ls'", *InFileStatus);
		State = EWorkspaceState::Unknown;
	}

	return State;
}

/**
 * Extract and interpret the file state from the cm "status" result.
 *
 * @param InResult One line of status from a "status" command
 * @param bInUsesCheckedOutChanged If using the new --iscochanged "CO+CH"
 * @return A workspace state
 *
 * Examples:
CO+CH;c:\Workspace\UEPlasticPluginDev\Content\Blueprints\CE_Game.uasset;False;NO_MERGES
MV;100%;c:\Workspace\UEPlasticPluginDev\Content\Blueprints\BP_ToRename.uasset;c:\Workspace\UEPlasticPluginDev\Content\Blueprints\BP_Renamed.uasset;False;NO_MERGES
 *
 * @see #ParseFileStatusResult() for more examples of results from "cm status --machinereadable"
*/
static FPlasticSourceControlState StateFromStatusResult(const FString& InResult, const bool bInUsesCheckedOutChanged)
{
	TArray<FString> ResultElements;
	InResult.ParseIntoArray(ResultElements, FILE_STATUS_SEPARATOR, false); // Don't cull empty values in csv
	if (ResultElements.Num() >= 4) // Note: should contain 4 or 6 elements (for moved files)
	{
		EWorkspaceState WorkspaceState = StateFromStatus(ResultElements[0], bInUsesCheckedOutChanged);
		if (WorkspaceState == EWorkspaceState::Moved)
		{
			// Special case for an asset that has been moved/renamed
			FString& File = ResultElements[3];
			FPlasticSourceControlState State(MoveTemp(File), WorkspaceState);
			State.MovedFrom = MoveTemp(ResultElements[2]);
			return State;
		}
		else
		{
			FString& File = ResultElements[1];
			return FPlasticSourceControlState(MoveTemp(File), WorkspaceState);
		}
	}

	UE_LOGF(LogSourceControl, Warning, "%ls", *InResult);

	return FPlasticSourceControlState(FString());
}

/**
 * @brief Parse status results in case of a regular operation for a list of files (not for a whole directory).
 *
 * This is the most common scenario, for any operation from the Content Browser or the View Changes window.
 *
 * In this case, iterates on the list of files the Editor provides,
 * searching corresponding file status from the array of strings results of a "status" command.
 *
 * @param[in]	InFiles		List of files in a directory (never empty).
 * @param[in]	InResults	Lines of results from the "status" command
 * @param[out]	OutStates	States of files for witch the status has been gathered
 *
 * Example of results from "cm status --machinereadable"
CH;c:\Workspace\UEPlasticPluginDev\Content\Changed_BP.uasset;False;NO_MERGES
CO;c:\Workspace\UEPlasticPluginDev\Content\CheckedOutUnchanged_BP.uasset;False;NO_MERGES
CO+CH;c:\Workspace\UEPlasticPluginDev\Content\CheckedOutChanged_BP.uasset;False;NO_MERGES
CO+CP;c:\Workspace\UEPlasticPluginDev\Content\Copied_BP.uasset;False;NO_MERGES
CO+RP;c:\Workspace\UEPlasticPluginDev\Content\Replaced_BP.uasset;False;NO_MERGES
AD;c:\Workspace\UEPlasticPluginDev\Content\Added_BP.uasset;False;NO_MERGES
PR;c:\Workspace\UEPlasticPluginDev\Content\Private_BP.uasset;False;NO_MERGES
IG;c:\Workspace\UEPlasticPluginDev\Content\Ignored_BP.uasset;False;NO_MERGES
DE;c:\Workspace\UEPlasticPluginDev\Content\Deleted_BP.uasset;False;NO_MERGES
LD;c:\Workspace\UEPlasticPluginDev\Content\Deleted2_BP.uasset;False;NO_MERGES
MV;100%;c:\Workspace\UEPlasticPluginDev\Content\ToMove_BP.uasset;c:\Workspace\UEPlasticPluginDev\Content\Moved_BP.uasset
 *
 * @see #ParseDirectoryStatusResult() that use a different parse logic
 */
void ParseFileStatusResult(TArray<FString>&& InFiles, const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& OutStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseFileStatusResult);

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const bool bUsesCheckedOutChanged = Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::StatusIsCheckedOutChanged;

	// Parse the list of status results in a map indexed by absolute filename
	TMap<FString, FPlasticSourceControlState> FileToStateMap;
	FileToStateMap.Reserve(InResults.Num());
	for (const FString& InResult : InResults)
	{
		FPlasticSourceControlState State = StateFromStatusResult(InResult, bUsesCheckedOutChanged);
		FileToStateMap.Add(State.LocalFilename, MoveTemp(State));
	}

	// Iterate on each file explicitly listed in the command
	for (FString& InFile : InFiles)
	{
		FPlasticSourceControlState FileState(MoveTemp(InFile));
		const FString& File = FileState.LocalFilename;

		// Search the file in the list of status
		if (FPlasticSourceControlState* State = FileToStateMap.Find(File))
		{
			// File found in status results; only the case for "changed" (or checked-out) files
			FileState.WorkspaceState = State->WorkspaceState;

			// Extract the original name of a Moved/Renamed file
			if (EWorkspaceState::Moved == FileState.WorkspaceState)
			{
				FileState.MovedFrom = State->MovedFrom;
			}
		}
		else
		{
			// File not found in status
			if (FPaths::FileExists(File))
			{
				// usually means the file is unchanged, or is on Hidden changes
				FileState.WorkspaceState = EWorkspaceState::Controlled; // Unchanged
			}
			else
			{
				// but also the case for newly created content: there is no file on disk until the content is saved for the first time (but we cannot mark is as locally deleted)
				FileState.WorkspaceState = EWorkspaceState::Private; // Not Controlled
			}
		}

		// debug log (only for the first few files)
		if (OutStates.Num() < 20)
		{
			UE_LOGF(LogSourceControl, Verbose, "%ls = %d:%ls", *File, static_cast<uint32>(FileState.WorkspaceState), FileState.ToString());
		}

		OutStates.Add(MoveTemp(FileState));
	}
	// debug log (if too many files)
	if (OutStates.Num() > 20)
	{
		UE_LOGF(LogSourceControl, Verbose, "[...] %d more files", OutStates.Num() - 20);
	}
}

/**
 * @brief Parse file status in case of a "whole directory status" (no file listed in the command).
 *
 * This is a less common scenario, typically calling the Submit Content, Revert All or Refresh commands
 * from the global source control menu.
 *
 * In this case, as there is no file list to iterate over,
 * just parse each line of the array of strings results from the "status" command.
 *
 * @param[in]	InDir		The path to the directory (never empty).
 * @param[in]	InResults	Lines of results from the "status" command
 * @param[out]	OutStates	States of files for witch the status has been gathered
 *
 * @see #ParseFileStatusResult() above for an example of a results from "cm status --machinereadable"
*/
void ParseDirectoryStatusResult(const FString& InDir, const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& OutStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseDirectoryStatusResult);

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const bool bUsesCheckedOutChanged = Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::StatusIsCheckedOutChanged;

	// First, find in the cache any existing states for files within the considered directory, that are not the default "Controlled" state
	TArray<FSourceControlStateRef> CachedStates = Provider.GetCachedStateByPredicate([&InDir](const FSourceControlStateRef& InState) {
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = StaticCastSharedRef<FPlasticSourceControlState>(InState);
		return (State->WorkspaceState != EWorkspaceState::Unknown) && (State->WorkspaceState != EWorkspaceState::Controlled) && InState->GetFilename().StartsWith(InDir);
	});

	// Iterate on each line of result of the status command
	for (const FString& InResult : InResults)
	{
		FPlasticSourceControlState FileState = StateFromStatusResult(InResult, bUsesCheckedOutChanged);
		if (!FileState.LocalFilename.IsEmpty())
		{
			UE_LOGF(LogSourceControl, Verbose, "%ls = %d:%ls", *FileState.LocalFilename, static_cast<uint32>(FileState.WorkspaceState), FileState.ToString());

			// If a new state has been found in the directory status, we will update the cached state for the file later, let's remove it from the list
			CachedStates.RemoveAll([&CachedStates, &FileState](FSourceControlStateRef& PreviousState) {
				return PreviousState->GetFilename().Equals(FileState.GetFilename(), ESearchCase::IgnoreCase);
			});

			OutStates.Add(MoveTemp(FileState));
		}
	}

	// Finally, update the cache for the files that where not found in the status results (eg checked-in or reverted outside of the Editor)
	for (const auto& CachedState : CachedStates)
	{
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = StaticCastSharedRef<FPlasticSourceControlState>(CachedState);
		// Check if a file that was "deleted" or "locally deleted" has been reverted or checked-in by testing if it still exists on disk
		if (State->IsDeleted() && !FPaths::FileExists(State->GetFilename()))
		{
			// Remove the file from the cache if it has been deleted from disk
			Provider.RemoveFileFromCache(State->GetFilename());
		}
		else
		{
			// Switch back the file state to the default Controlled status (Unknown would prevent checkout)
			State->WorkspaceState = EWorkspaceState::Controlled;
		}

		// also remove the file from its changelist if any
		if (State->Changelist.IsInitialized())
		{
			// 1- Remove these files from their previous changelist
			TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> ChangelistState = Provider.GetStateInternal(State->Changelist);
			ChangelistState->Files.Remove(State);
			// 2- And reset the reference to their previous changelist
			State->Changelist.Reset();
		}
	}
}

/// Visitor to list all files in subdirectory
class FFileVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory)
		{
			Files.Add(FilenameOrDirectory);
		}
		return true;
	}

	TArray<FString> Files;
};

FPlasticSourceControlLock ParseLockInfo(const FString& InResult)
{
	FPlasticSourceControlLock Lock;
	TArray<FString> SmartLockInfos;
	const int32 NbElmts = InResult.ParseIntoArray(SmartLockInfos, FILE_STATUS_SEPARATOR, false);
	if (NbElmts >= 12)
	{
		Lock.ItemId = FCString::Atoi(*SmartLockInfos[1]);
		FDateTime::ParseIso8601(*SmartLockInfos[3], Lock.Date);
		Lock.DestinationBranch = MoveTemp(SmartLockInfos[4]);
		Lock.Branch = MoveTemp(SmartLockInfos[6]);
		Lock.Status = MoveTemp(SmartLockInfos[8]);
		Lock.bIsLocked = (Lock.Status == TEXT("Locked"));
		// Note: keeping the full email address as the owner name so we can display both the short and full name in the tooltip
		Lock.Owner = MoveTemp(SmartLockInfos[9]);
		Lock.Workspace = MoveTemp(SmartLockInfos[10]);
		Lock.Path = MoveTemp(SmartLockInfos[11]);
	}
	return Lock;
}

// Parse the fileinfo output format "{RevisionChangeset};{RevisionHeadChangeset};{RepSpec};{LockedBy};{LockedWhere};{ServerPath}"
// for example "40;41;repo@server:port;srombauts;UEPlasticPluginDev"
class FPlasticFileinfoParser
{
public:
	explicit FPlasticFileinfoParser(const FString& InResult)
	{
		TArray<FString> Fileinfos;
		InResult.ParseIntoArray(Fileinfos, TEXT(";"), false); // Don't cull empty values in csv
		if (Fileinfos.Num() == 6)
		{
			RevisionChangeset = FCString::Atoi(*Fileinfos[0]);
			RevisionHeadChangeset = FCString::Atoi(*Fileinfos[1]);
			RepSpec = MoveTemp(Fileinfos[2]);
			LockedBy = PlasticSourceControlUtils::UserNameToDisplayName(MoveTemp(Fileinfos[3]));
			LockedWhere = MoveTemp(Fileinfos[4]);
			ServerPath = MoveTemp(Fileinfos[5]);
		}
	}

	int32 RevisionChangeset;
	int32 RevisionHeadChangeset;
	FString RepSpec;
	FString LockedBy;
	FString LockedWhere;
	FString ServerPath;
};

/**
 * Find the locks matching the file path from the list of locks
 *
 * Multiple matching locks can only happen if multiple destination branches are configured
*/
TArray<FPlasticSourceControlLockRef> FindMatchingLocks(const TArray<FPlasticSourceControlLockRef>& InLocks, const FString& InPath)
{
	TArray<FPlasticSourceControlLockRef> MatchingLocks;
	for (int i = 0; i < InLocks.Num(); i++)
	{
		if (InLocks[i]->Path == InPath)
		{
			MatchingLocks.Add(InLocks[i]);
		}
	}
	return MatchingLocks;
}

void ConcatStrings(FString& InOutString, const TCHAR* InSeparator, const FString& InOther)
{
	if (!InOutString.IsEmpty())
	{
		InOutString += InSeparator;
	}
	InOutString += InOther;
}

/** Parse the array of strings result of a 'cm fileinfo --format="{RevisionChangeset};{RevisionHeadChangeset};{RepSpec};{LockedBy};{LockedWhere}"' command
 *
 * Example cm fileinfo results:
16;16;;
14;15;;
17;17;srombauts;Workspace_2
 */
void ParseFileinfoResults(const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& InOutStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseFileinfoResults);

	ensureMsgf(InResults.Num() == InOutStates.Num(), TEXT("The fileinfo command should gives the same number of infos as the status command"));

	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	// Note: here is one of the rare places where we need to use a branch name, not a workspace selector
	const FString& BranchName = Provider.GetBranchName();

	TArray<FPlasticSourceControlLockRef> Locks;
	if (Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::SmartLocks)
	{
		// In the Content Browser, only show locks applying to the current working branch
		const bool bForAllDestBranches = false;
		PlasticSourceControlUtils::RunListLocks(Provider, bForAllDestBranches, Locks);
	}

	// Iterate on all files and all status of the result (assuming same number of line of results than number of file states)
	for (int32 IdxResult = 0; IdxResult < InResults.Num(); IdxResult++)
	{
		const FString& Fileinfo = InResults[IdxResult];
		FPlasticSourceControlState& FileState = InOutStates[IdxResult];
		const FString& File = FileState.LocalFilename;
		FPlasticFileinfoParser FileinfoParser(Fileinfo);

		FileState.LocalRevisionChangeset = FileinfoParser.RevisionChangeset;
		FileState.DepotRevisionChangeset = FileinfoParser.RevisionHeadChangeset;
		FileState.RepSpec = FileinfoParser.RepSpec;

		// Additional information coming from Locks (branch, workspace, date and lock status)
		// Note: in case of multi destination branches, we might have multiple locks for the same path, so we concatenate the string info
		const TArray<FPlasticSourceControlLockRef> MatchingLocks = FindMatchingLocks(Locks, FileinfoParser.ServerPath);
		for (auto& Lock : MatchingLocks)
		{
			// "Locked" vs "Retained" lock
			if (Lock->bIsLocked)
			{
				ConcatStrings(FileState.LockedBy, TEXT(", "), PlasticSourceControlUtils::UserNameToDisplayName(Lock->Owner));
			}
			// Considers a "Retained" lock as meaningful only if it is retained on another branch
			// NOTE: this is required to avoid the Unreal Editor showing a popup warning preventing the user to save the asset
			else if (Lock->Branch != BranchName)
			{
				ConcatStrings(FileState.RetainedBy, TEXT(", "), PlasticSourceControlUtils::UserNameToDisplayName(Lock->Owner));
			}
			ConcatStrings(FileState.LockedWhere, TEXT(", "), Lock->Workspace);
			ConcatStrings(FileState.LockedBranch, TEXT(", "), Lock->Branch);

			// Only save the ItemId if there is only one matching Lock: used to Unlock it from the context menu in the Content Browser,
			// but leave the ItmeId to invalid if there are more than one: there would be no way to know which one to unlock from the context menu
			// (Unlocking in such a case require using the View Locks window instead for disambiguation)
			if (MatchingLocks.Num() == 1)
			{
				FileState.LockedId = Lock->ItemId;
			}
			// Note; this will keep only the date of the last lock
			FileState.LockedDate = Lock->Date;
		}

		// debug log (only for the first few files)
		if (IdxResult < 20)
		{
			UE_LOGF(LogSourceControl, Verbose, "%ls: %d;%d %ls by '%ls' (%ls)", *File, FileState.LocalRevisionChangeset, FileState.DepotRevisionChangeset, *FileState.RepSpec, *FileState.LockedBy, *FileState.LockedWhere);
		}
	}
	// debug log (if too many files)
	if (InResults.Num() > 20)
	{
		UE_LOGF(LogSourceControl, Verbose, "[...] %d more files", InResults.Num() - 20);
	}
}

// FILE_CONFLICT /Content/FirstPersonBP/Blueprints/FirstPersonProjectile.uasset 1 4 6 903
// (explanations: 'The file /Content/FirstPersonBP/Blueprints/FirstPersonProjectile.uasset needs to be merged from cs:4 to cs:6 base cs:1. Changed by both contributors.')
FPlasticMergeConflictParser::FPlasticMergeConflictParser(const FString& InResult)
{
	static const FString FILE_CONFLICT(TEXT("FILE_CONFLICT "));
	if (InResult.StartsWith(FILE_CONFLICT, ESearchCase::CaseSensitive))
	{
		FString Temp = InResult.RightChop(FILE_CONFLICT.Len());
		int32 WhitespaceIndex;
		if (Temp.FindChar(TEXT(' '), WhitespaceIndex))
		{
			Filename = Temp.Left(WhitespaceIndex);
		}
		Temp.RightChopInline(WhitespaceIndex + 1);
		if (Temp.FindChar(TEXT(' '), WhitespaceIndex))
		{
			BaseChangeset = Temp.Left(WhitespaceIndex);
		}
		Temp.RightChopInline(WhitespaceIndex + 1);
		if (Temp.FindChar(TEXT(' '), WhitespaceIndex))
		{
			SourceChangeset = Temp.Left(WhitespaceIndex);
		}
	}
}

// Types of changes in source control revisions, using Perforce terminology for the History window
static const TCHAR* SourceControlActionAdded = TEXT("add");
static const TCHAR* SourceControlActionDeleted = TEXT("delete");
static const TCHAR* SourceControlActionMoved = TEXT("branch");
static const TCHAR* SourceControlActionMerged = TEXT("integrate");
static const TCHAR* SourceControlActionChanged = TEXT("edit");

// Convert a file state to a string ala Perforce, see also ParseShelveFileStatus()
FString FileStateToAction(const EWorkspaceState InState)
{
	switch (InState)
	{
	case EWorkspaceState::Added:
		return SourceControlActionAdded;
	case EWorkspaceState::Deleted:
		return SourceControlActionDeleted;
	case EWorkspaceState::Moved:
	case EWorkspaceState::Copied:
		return SourceControlActionMoved;
	case EWorkspaceState::Replaced:
		return SourceControlActionMerged;
	case EWorkspaceState::CheckedOutChanged:
	default:
		return SourceControlActionChanged;
	}
}

// TODO PR to move this in Engine
static FString DecodeXmlEntities(const FString& InString)
{
	FString String = InString;
	int32 AmpIdx;
	if (String.FindChar(TEXT('&'), AmpIdx))
	{
		String.ReplaceInline(TEXT("&amp;"), TEXT("&"), ESearchCase::CaseSensitive);
		String.ReplaceInline(TEXT("&quot;"), TEXT("\""), ESearchCase::CaseSensitive);
		String.ReplaceInline(TEXT("&apos;"), TEXT("'"), ESearchCase::CaseSensitive);
		String.ReplaceInline(TEXT("&lt;"), TEXT("<"), ESearchCase::CaseSensitive);
		String.ReplaceInline(TEXT("&gt;"), TEXT(">"), ESearchCase::CaseSensitive);
	}
	return String;
}

/**
 * Parse one specific node of the result of the 'cm history --moveddeleted' command.
 *
 * Results of the history command in case of a move looks like that:
 <Branch>Moved from /Content/FirstPersonBP/Blueprints/BP_ToRename.uasset to /Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset</Branch>
*/
static FString ParseMovedFrom(const FXmlNode* InBranchNode)
{
	FString MovedFrom;

	if (InBranchNode != nullptr)
	{
		static const int32 MovedFromPrefixLen = FString("Moved from /").Len();
		MovedFrom = InBranchNode->GetContent().RightChop(MovedFromPrefixLen);

		const int32 MovedToIndex = MovedFrom.Find(TEXT(" to "), ESearchCase::CaseSensitive);
		if (MovedToIndex != INDEX_NONE)
		{
			MovedFrom.LeftInline(MovedToIndex);
		}
	}

	return MovedFrom; // Convert server path to absolute
}

/**
 * Parse results of the 'cm history --moveddeleted --xml --encoding="utf-8"' command.
 *
 * Results of the history command looks like that:
<RevisionHistoriesResult>
  <RevisionHistories>
	<RevisionHistory>
	  <ItemName>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset</ItemName>
	  <Revisions>
		<Revision>
		  <RevisionSpec>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset#cs:7</RevisionSpec>
		  <Branch>/main</Branch>
		  <CreationDate>2019-10-14T09:52:07+02:00</CreationDate>
		  <RevisionType>bin</RevisionType>
		  <ChangesetNumber>7</ChangesetNumber>
		  <Owner>sebastien.rombauts</Owner>
		  <Comment>New tests</Comment>
		  <Repository>UE4PlasticPluginDev</Repository>
		  <Server>localhost:8087</Server>
		  <RepositorySpec>UE4PlasticPluginDev@localhost:8087</RepositorySpec>
		  <DataStatus>Available</DataStatus>
		  <ItemId>1657</ItemId>
		  <Size>22356</Size>
		  <Hash>zzuB6G9fbWz1md12+tvBxg==</Hash>
		</Revision>
		...
		<Revision>
		  <RevisionSpec>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset#cs:12</RevisionSpec>
		  <Branch>/main/rename_test</Branch>
		  <CreationDate>2022-04-28T16:00:37+02:00</CreationDate>
		  <RevisionType>bin</RevisionType>
		  <ChangesetNumber>12</ChangesetNumber>
		  <Owner>sebastien.rombauts</Owner>
		  <Comment>Renamed sphere blueprint</Comment>
		  <Repository>UE4PlasticPluginDev</Repository>
		  <Server>localhost:8087</Server>
		  <RepositorySpec>UE4PlasticPluginDev@localhost:8087</RepositorySpec>
		  <DataStatus>Available</DataStatus>
		  <ItemPathOrSpec>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset</ItemPathOrSpec>
		  <ItemId>1657</ItemId>
		  <Size>28603</Size>
		  <Hash>MREVIZ1qKNqu1h2iq9WiRg==</Hash>
		</Revision>
		<Revision>
		  <RevisionSpec>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset#cs:12</RevisionSpec>
		  <Branch>Moved from /Content/FirstPersonBP/Blueprints/BP_ToRename.uasset to /Content/FirstPersonBP/Blueprints/BP_TestsRenamed.uasset</Branch>
		  <CreationDate>2022-04-28T16:00:37+02:00</CreationDate>
		  <RevisionType />
		  <ChangesetNumber>12</ChangesetNumber>
		  <Owner>sebastien.rombauts</Owner>
		  <Comment />
		  <Repository>UE4PlasticPluginDev</Repository>
		  <Server>localhost:8087</Server>
		  <RepositorySpec>UE4PlasticPluginDev@localhost:8087</RepositorySpec>
		  <DataStatus />
		  <ItemPathOrSpec />
		  <ItemId />
		  <Size>0</Size>
		  <Hash />
		</Revision>
		...
	  </Revisions>
	</RevisionHistory>
	<RevisionHistory>
	  <ItemName>C:/Workspace/UE4PlasticPluginDev/Content/FirstPersonBP/Blueprints/BP_YetAnother.uasset</ItemName>
		...
	</RevisionHistory>
  </RevisionHistories>
</RevisionHistoriesResult>
*/
static bool ParseHistoryResults(const bool bInUpdateHistory, const FXmlFile& InXmlResult, TArray<FPlasticSourceControlState>& InOutStates)
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const FString& WorkspaceRoot = Provider.GetPathToWorkspaceRoot();
	const FString RootRepSpec = FString::Printf(TEXT("%s@%s"), *Provider.GetRepositoryName(), *Provider.GetServerUrl());
	const FString CurrentBranch = Provider.GetBranchName();

	static const FString RevisionHistoriesResult(TEXT("RevisionHistoriesResult"));
	static const FString RevisionHistories(TEXT("RevisionHistories"));
	static const FString RevisionHistory(TEXT("RevisionHistory"));
	static const FString ItemName(TEXT("ItemName"));
	static const FString Revisions(TEXT("Revisions"));
	static const FString Revision(TEXT("Revision"));
	static const FString Branch(TEXT("Branch"));
	static const FString CreationDate(TEXT("CreationDate"));
	static const FString RevisionType(TEXT("RevisionType"));
	static const FString ChangesetNumber(TEXT("ChangesetNumber"));
	static const FString Owner(TEXT("Owner"));
	static const FString Comment(TEXT("Comment"));
	static const FString Size(TEXT("Size"));
	static const FString Hash(TEXT("Hash"));

	const FXmlNode* RevisionHistoriesResultNode = InXmlResult.GetRootNode();
	if (RevisionHistoriesResultNode == nullptr || RevisionHistoriesResultNode->GetTag() != RevisionHistoriesResult)
	{
		return false;
	}

	const FXmlNode* RevisionHistoriesNode = RevisionHistoriesResultNode->FindChildNode(RevisionHistories);
	if (RevisionHistoriesNode == nullptr)
	{
		return false;
	}

	const TArray<FXmlNode*>& RevisionHistoryNodes = RevisionHistoriesNode->GetChildrenNodes();
	for (const FXmlNode* RevisionHistoryNode : RevisionHistoryNodes)
	{
		const FXmlNode* ItemNameNode = RevisionHistoryNode->FindChildNode(ItemName);
		if (ItemNameNode == nullptr)
		{
			continue;
		}

		FString Filename = ItemNameNode->GetContent();
		FPlasticSourceControlState* InOutStatePtr = InOutStates.FindByPredicate(
			[&Filename](const FPlasticSourceControlState& State) { return State.LocalFilename == Filename; }
		);
		if (InOutStatePtr == nullptr)
		{
			continue;
		}
		FPlasticSourceControlState& InOutState = *InOutStatePtr;

		const FXmlNode* RevisionsNode = RevisionHistoryNode->FindChildNode(Revisions);
		if (RevisionsNode == nullptr)
		{
			continue;
		}

		const TArray<FXmlNode*>& RevisionNodes = RevisionsNode->GetChildrenNodes();
		if (bInUpdateHistory)
		{
			InOutState.History.Reserve(RevisionNodes.Num());
		}

		// parse history in reverse: needed to get most recent at the top (required by Unreal Editor for the "Diff with depot" using the index 0)
		FString NextEntryMovedFrom;
		for (int32 RevisionIndex = RevisionNodes.Num() - 1; RevisionIndex >= 0; RevisionIndex--)
		{
			const FXmlNode* RevisionNode = RevisionNodes[RevisionIndex];
			check(RevisionNode);

			const TSharedRef<FPlasticSourceControlRevision, ESPMode::ThreadSafe> SourceControlRevision = MakeShareable(new FPlasticSourceControlRevision);
			SourceControlRevision->State = &InOutState;
			SourceControlRevision->Filename = Filename;

			if (const FXmlNode* RevisionTypeNode = RevisionNode->FindChildNode(RevisionType))
			{
				// There are two entries for a Move of an asset;
				// 1. a regular one with the normal data: revision, comment, branch, Id, size, hash etc.
				// 2. and another "empty" one for the Move
				// => Since the parsing is done in reverse order, the detection of a Move need to apply to the next entry
				if (RevisionTypeNode->GetContent().IsEmpty())
				{
					// An empty <RevisionType> signals a Move: save the "MovedFrom" filename to treat the next entry as a Move and update the Filename accordingly for next (older) entries
					NextEntryMovedFrom = FPaths::Combine(WorkspaceRoot, ParseMovedFrom(RevisionNode->FindChildNode(Branch)));

					// and skip this revision as it is empty (it's just an additional entry with data for the move)
					continue;
				}
				else
				{
					// If this entry was flagged as a move:
					if (NextEntryMovedFrom.Len() > 0)
					{
						// Set this revision as a Move
						SourceControlRevision->Action = SourceControlActionMoved;
						// Update Filename for next (older) entries in the history and clear the NextEntryMovedFrom used as a flag
						Filename = MoveTemp(NextEntryMovedFrom);
					}
					else if (RevisionIndex == 0)
					{
						SourceControlRevision->Action = SourceControlActionAdded;
					}
					else
					{
						SourceControlRevision->Action = SourceControlActionChanged;
					}
				}
			}

			if (const FXmlNode* ChangesetNumberNode = RevisionNode->FindChildNode(ChangesetNumber))
			{
				const FString& Changeset = ChangesetNumberNode->GetContent();
				SourceControlRevision->ChangesetNumber = FCString::Atoi(*Changeset); // Value now used in the Revision column and in the Asset Menu History

				// Also append depot name to the revision, but only when it is different from the default one (ie for xlinks sub repository)
				if (!InOutState.RepSpec.IsEmpty() && (InOutState.RepSpec != RootRepSpec))
				{
					TArray<FString> RepSpecs;
					InOutState.RepSpec.ParseIntoArray(RepSpecs, TEXT("@"));
					SourceControlRevision->Revision = FString::Printf(TEXT("cs:%s@%s"), *Changeset, *RepSpecs[0]);
				}
				else
				{
					SourceControlRevision->Revision = FString::Printf(TEXT("cs:%s"), *Changeset);
				}
			}
			if (const FXmlNode* CommentNode = RevisionNode->FindChildNode(Comment))
			{
				SourceControlRevision->Description = DecodeXmlEntities(CommentNode->GetContent());
			}
			if (const FXmlNode* OwnerNode = RevisionNode->FindChildNode(Owner))
			{
				SourceControlRevision->UserName = PlasticSourceControlUtils::UserNameToDisplayName(OwnerNode->GetContent());
			}
			if (const FXmlNode* DateNode = RevisionNode->FindChildNode(CreationDate))
			{
				const FString& DateIso = DateNode->GetContent();
				FDateTime::ParseIso8601(*DateIso, SourceControlRevision->Date);
			}
			if (const FXmlNode* BranchNode = RevisionNode->FindChildNode(Branch))
			{
				SourceControlRevision->Branch = DecodeXmlEntities(BranchNode->GetContent());
			}
			if (const FXmlNode* SizeNode = RevisionNode->FindChildNode(Size))
			{
				SourceControlRevision->FileSize = FCString::Atoi(*SizeNode->GetContent());
			}

			// A negative RevisionHeadChangeset provided by fileinfo mean that the file has been unshelved;
			// replace it by the changeset number of the first revision in the history (the more recent)
			// Note: workaround to be able to show the history / the diff of a file that has been unshelved
			// (but keeps the LocalRevisionChangeset to the negative changeset corresponding to the Shelve Id)
			if (InOutState.DepotRevisionChangeset < 0)
			{
				InOutState.DepotRevisionChangeset = SourceControlRevision->ChangesetNumber;
			}

			// Detect and skip more recent changesets on other branches (ie above the RevisionHeadChangeset)
			// since we usually don't want to display changes from other branches in the History window...
			// except in case of a merge conflict, where the Editor expects the tip of the "source (remote)" branch to be at the top of the history!
			if (   (SourceControlRevision->ChangesetNumber > InOutState.DepotRevisionChangeset)
				&& (SourceControlRevision->Branch != CurrentBranch)
				&& (SourceControlRevision->GetRevision() != InOutState.PendingResolveInfo.RemoteRevision))
			{
				InOutState.HeadBranch = SourceControlRevision->Branch;
				InOutState.HeadAction = SourceControlRevision->Action;
				InOutState.HeadChangeList = SourceControlRevision->ChangesetNumber;
				InOutState.HeadUserName = SourceControlRevision->UserName;
				InOutState.HeadModTime = SourceControlRevision->Date.ToUnixTimestamp();
			}
			else if (bInUpdateHistory)
			{
				InOutState.History.Add(SourceControlRevision);
			}

			// Also grab the UserName of the author of the current depot/head changeset
			if ((SourceControlRevision->ChangesetNumber == InOutState.DepotRevisionChangeset) && InOutState.HeadUserName.IsEmpty())
			{
				InOutState.HeadUserName = SourceControlRevision->UserName;
			}

			if (!bInUpdateHistory)
			{
				break; // if not updating the history, just getting the head of the latest branch is enough
			}
		}
	}

	return true;
}

bool ParseHistoryResults(const bool bInUpdateHistory, const FString& InXmlFilename, TArray<FPlasticSourceControlState>& InOutStates)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseHistoryResults::LoadXml);
		bResult = XmlFile.LoadFile(InXmlFilename);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseHistoryResults::ParseXml);
		bResult = ParseHistoryResults(bInUpdateHistory, XmlFile, InOutStates);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseHistoryResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

/* Parse results of the 'cm update --xml=tempfile.xml --encoding="utf-8"' command.
 *
 * Results of the update command looks like that:
<UpdatedItems>
  <List>
	<UpdatedItem>
	  <Path>c:\Workspace\UE5PlasticPluginDev\Content\NewFolder\BP_CheckedOut.uasset</Path>
	  <User>sebastien.rombauts@unity3d.com</User>
	  <Changeset>94</Changeset>
	  <Date>2022-10-27T11:58:02+02:00</Date>
	</UpdatedItem>
  </List>
</UpdatedItems>
*/
static bool ParseUpdateResults(const FXmlFile& InXmlResult, TArray<FString>& OutFiles)
{
	static const FString UpdatedItems(TEXT("UpdatedItems"));
	static const FString List(TEXT("List"));
	static const FString UpdatedItem(TEXT("UpdatedItem"));
	static const FString Path(TEXT("Path"));

	const FXmlNode* UpdatedItemsNode = InXmlResult.GetRootNode();
	if (UpdatedItemsNode == nullptr || UpdatedItemsNode->GetTag() != UpdatedItems)
	{
		return false;
	}

	const FXmlNode* ListNode = UpdatedItemsNode->FindChildNode(List);
	if (ListNode == nullptr)
	{
		return false;
	}

	const TArray<FXmlNode*>& UpdatedItemNodes = ListNode->GetChildrenNodes();
	for (const FXmlNode* UpdatedItemNode : UpdatedItemNodes)
	{
		if (const FXmlNode* PathNode = UpdatedItemNode->FindChildNode(Path))
		{
			FString Filename = PathNode->GetContent();
			FPaths::NormalizeFilename(Filename);
			if (!OutFiles.Contains(Filename))
			{
				OutFiles.Add(Filename);
			}
		}
	}

	return true;
}

bool ParseUpdateResults(const FString& InResults, TArray<FString>& OutFiles)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseUpdateResults::LoadXml);
		bResult = XmlFile.LoadFile(InResults, EConstructMethod::ConstructFromBuffer);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseUpdateResults::ParseXml);
		bResult = ParseUpdateResults(XmlFile, OutFiles);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseUpdateResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

/* Parse results of the 'cm partial update --report --machinereadable' command.
 *
 * Results of the update command looks like that:
STAGE Plastic is updating your workspace. Wait a moment, please...
STAGE Updated 63.01 KB of 63.01 KB (12 of 12 files to download / 16 of 21 operations to apply) /Content/Collections/SebSharedCollection.collection
AD c:\Workspace\UE5PlasticPluginDev\Content\LevelPrototyping\Materials\MI_Solid_Red.uasset
CH c:\Workspace\UE5PlasticPluginDev\Config\DefaultEditor.ini
DE c:\Workspace\UE5PlasticPluginDev\Content\Collections\SebSharedCollection.collection
*/
bool ParseUpdateResults(const TArray<FString>& InResults, TArray<FString>& OutFiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseUpdateResultsString);

	static const FString Stage = TEXT("STAGE ");
	static const int32 PrefixLen = 3; // "XX " typically "CH ", "AD " or "DE "

	for (const FString& Result : InResults)
	{
		if (Result.StartsWith(Stage))
			continue;

		FString Filename = Result.RightChop(PrefixLen);
		FPaths::NormalizeFilename(Filename);
		if (!OutFiles.Contains(Filename))
		{
			OutFiles.Add(Filename);
		}
	}

	return true;
}


/// Parse checkin result, usually looking like "Created changeset cs:8@br:/main@MyProject@SRombauts@cloud (mount:'/')"
FText ParseCheckInResults(const TArray<FString>& InResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseCheckInResults);

	if (InResults.Num() > 0)
	{
		static const FString ChangesetPrefix(TEXT("Created changeset "));
		if (InResults.Last().StartsWith(ChangesetPrefix))
		{
			FString ChangesetString;
			static const FString BranchPrefix(TEXT("@br:"));
			const int32 BranchIndex = InResults.Last().Find(BranchPrefix, ESearchCase::CaseSensitive);
			if (BranchIndex > INDEX_NONE)
			{
				ChangesetString = InResults.Last().Mid(ChangesetPrefix.Len(), BranchIndex - ChangesetPrefix.Len());
			}
			return FText::Format(NSLOCTEXT("PlasticSourceControl", "SubmitMessage", "Submitted changeset {0}"), FText::FromString(ChangesetString));
		}
		else
		{
			return FText::FromString(InResults.Last());
		}
	}
	return FText();
}

/**
 * Parse results of the 'cm status --changelists --controlledchanged --noheader --xml --encoding="utf-8"' command.
 *
 * Results of the status changelists command looks like that:
<StatusOutput>
  <WkConfigType>Branch</WkConfigType>
  <WkConfigName>/main@rep:UEPlasticPluginDev@repserver:test@cloud</WkConfigName>
  <Changelists>
	<Changelist>
	  <Name>Default</Name>
	  <Description>Default Unity Version Control changelist</Description>
	  <Changes>
		<Change>
		  <Type>CO</Type>
		  <TypeVerbose>Checked-out</TypeVerbose>
		  <Path>UEPlasticPluginDev.uproject</Path>
		  <OldPath />
		  <PrintableMovedPath />
		  <MergesInfo />
		  <SimilarityPerUnit>0</SimilarityPerUnit>
		  <Similarity />
		  <Size>583</Size>
		  <PrintableSize>583 bytes</PrintableSize>
		  <PrintableLastModified>6 days ago</PrintableLastModified>
		  <RevisionType>enTextFile</RevisionType>
		  <LastModified>2022-06-07T12:28:32+02:00</LastModified>
		</Change>
		[...]
		<Change>
		</Change>
	  </Changes>
	</Changelist>
  </Changelists>
</StatusOutput>
*/
static bool ParseChangelistsResults(const FXmlFile& InXmlResult, TArray<FPlasticSourceControlChangelistState>& OutChangelistsStates, TArray<TArray<FPlasticSourceControlState>>& OutCLFilesStates)
{
	static const FString StatusOutput(TEXT("StatusOutput"));
	static const FString WkConfigType(TEXT("WkConfigType"));
	static const FString WkConfigName(TEXT("WkConfigName"));
	static const FString Changelists(TEXT("Changelists"));
	static const FString Changelist(TEXT("Changelist"));
	static const FString Name(TEXT("Name"));
	static const FString Description(TEXT("Description"));
	static const FString Changes(TEXT("Changes"));
	static const FString Change(TEXT("Change"));
	static const FString Type(TEXT("Type"));
	static const FString Path(TEXT("Path"));
	static const FString OldPath(TEXT("OldPath"));

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const FString& WorkspaceRoot = Provider.GetPathToWorkspaceRoot();
	const bool bUsesCheckedOutChanged = Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::StatusIsCheckedOutChanged;

	const FXmlNode* StatusOutputNode = InXmlResult.GetRootNode();
	if (StatusOutputNode == nullptr || StatusOutputNode->GetTag() != StatusOutput)
	{
		return false;
	}

	const FXmlNode* ChangelistsNode = StatusOutputNode->FindChildNode(Changelists);
	if (ChangelistsNode)
	{
		const TArray<FXmlNode*>& ChangelistNodes = ChangelistsNode->GetChildrenNodes();
		OutCLFilesStates.SetNum(ChangelistNodes.Num());
		for (int32 ChangelistIndex = 0; ChangelistIndex < ChangelistNodes.Num(); ChangelistIndex++)
		{
			const FXmlNode* ChangelistNode = ChangelistNodes[ChangelistIndex];
			check(ChangelistNode);
			const FXmlNode* NameNode = ChangelistNode->FindChildNode(Name);
			const FXmlNode* DescriptionNode = ChangelistNode->FindChildNode(Description);
			const FXmlNode* ChangesNode = ChangelistNode->FindChildNode(Changes);
			if (NameNode == nullptr || DescriptionNode == nullptr || ChangesNode == nullptr)
			{
				continue;
			}

			FString NameTemp = DecodeXmlEntities(NameNode->GetContent());
			FPlasticSourceControlChangelist ChangelistTemp(MoveTemp(NameTemp), true);
			FString DescriptionTemp = ChangelistTemp.IsDefault() ? FString() : DecodeXmlEntities(DescriptionNode->GetContent());
			FPlasticSourceControlChangelistState ChangelistState(MoveTemp(ChangelistTemp), MoveTemp(DescriptionTemp));

			const TArray<FXmlNode*>& ChangeNodes = ChangesNode->GetChildrenNodes();
			for (const FXmlNode* ChangeNode : ChangeNodes)
			{
				check(ChangeNode);
				const FXmlNode* PathNode = ChangeNode->FindChildNode(Path);
				if (PathNode == nullptr)
				{
					continue;
				}

				FPlasticSourceControlState FileState(FPaths::ConvertRelativePathToFull(WorkspaceRoot, PathNode->GetContent()));
				FileState.Changelist = ChangelistState.Changelist;
				if (const FXmlNode* TypeNode = ChangeNode->FindChildNode(Type))
				{
					FileState.WorkspaceState = StateFromStatus(TypeNode->GetContent(), bUsesCheckedOutChanged);
				}

				if (FileState.WorkspaceState == EWorkspaceState::Moved)
				{
					if (const FXmlNode* OldPathNode = ChangeNode->FindChildNode(OldPath))
					{
						FileState.MovedFrom = FPaths::ConvertRelativePathToFull(WorkspaceRoot, OldPathNode->GetContent());
					}
				}

				// Note: in case of a Moved file, it appears twice in the list; just update the first entry (set as a "Changed") with the "Move" status
				if (FPlasticSourceControlState* ExistingState = OutCLFilesStates[ChangelistIndex].FindByPredicate(
					[&FileState](const FPlasticSourceControlState& InState)
					{
						return InState.GetFilename().Equals(FileState.GetFilename());
					}))
				{
					ExistingState->WorkspaceState = FileState.WorkspaceState;
					ExistingState->MovedFrom = FileState.MovedFrom;
				}
				else
				{
					OutCLFilesStates[ChangelistIndex].Add(MoveTemp(FileState));
				}
			}

			OutChangelistsStates.Add(ChangelistState);
		}
	}

	if (!OutChangelistsStates.FindByPredicate(
		[](const FPlasticSourceControlChangelistState& CLState) { return CLState.Changelist.IsDefault(); }
	))
	{
		// No Default Changelists isn't an error, but the Editor UX expects to always the Default changelist (so you can always move files back to it)
		FPlasticSourceControlChangelistState DefaultChangelistState(FPlasticSourceControlChangelist::DefaultChangelist);
		OutChangelistsStates.Insert(DefaultChangelistState, 0);
		OutCLFilesStates.Insert(TArray<FPlasticSourceControlState>(), 0);
	}

	return true;
}

bool ParseChangelistsResults(const FString& InXmlFilename, TArray<FPlasticSourceControlChangelistState>& OutChangelistsStates, TArray<TArray<FPlasticSourceControlState>>& OutCLFilesStates)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseChangelistsResults::LoadXml);
		bResult = XmlFile.LoadFile(InXmlFilename);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseChangelistsResults::ParseXml);
		bResult = ParseChangelistsResults(XmlFile, OutChangelistsStates, OutCLFilesStates);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseChangelistsResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

// Parse the one letter file status in front of each line of the 'cm diff sh:<ShelveId>'
EWorkspaceState ParseShelveFileStatus(const TCHAR InFileStatus)
{
	if (InFileStatus == 'A') // Added
	{
		return EWorkspaceState::Added;
	}
	else if (InFileStatus == 'D') // Deleted
	{
		return EWorkspaceState::Deleted;
	}
	else if (InFileStatus == 'C') // Changed (CheckedOut or not)
	{
		return EWorkspaceState::CheckedOutChanged;
	}
	else if (InFileStatus == 'M') // Moved/Renamed (or Locally Moved)
	{
		return EWorkspaceState::Moved;
	}
	else
	{
		UE_LOGF(LogSourceControl, Warning, "Unknown file status '%c'", InFileStatus);
		return EWorkspaceState::Unknown;
	}
}

/**
 * Parse results of the 'cm diff sh:<ShelveId>' command.
 *
 * Results of the diff command looks like that:
C "Content\NewFolder\BP_CheckedOut.uasset"
C "Content\NewFolder\BP_Renamed.uasset"
A "Content\NewFolder\BP_ControlledUnchanged.uasset"
D "Content\NewFolder\BP_Changed.uasset"
M "Content\NewFolder\BP_ControlledUnchanged.uasset" "Content\NewFolder\BP_Renamed.uasset"
*/
bool ParseShelveDiffResult(const FString InWorkspaceRoot, TArray<FString>&& InResults, FPlasticSourceControlChangelistState& InOutChangelistsState)
{
	bool bResult = true;

	InOutChangelistsState.ShelvedFiles.Reset(InResults.Num());
	for (FString& Result : InResults)
	{
		EWorkspaceState ShelveState = ParseShelveFileStatus(Result[0]);

		// Remove outer double quotes
		Result.MidInline(3, Result.Len() - 4, EAllowShrinking::No);

		FString MovedFrom;
		if (ShelveState == EWorkspaceState::Moved)
		{
			// Search for the inner double quotes in the middle of "Content/Source.uasset" "Content/Destination.uasset" to keep only the destination filename
			int32 RenameIndex;
			if (Result.FindLastChar(TEXT('"'), RenameIndex))
			{
				MovedFrom = Result.Left(RenameIndex - 2);
				MovedFrom = FPaths::ConvertRelativePathToFull(InWorkspaceRoot, MovedFrom);
				Result.RightChopInline(RenameIndex + 1);
			}
		}

		if (ShelveState != EWorkspaceState::Unknown && !Result.IsEmpty())
		{
			FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InWorkspaceRoot, MoveTemp(Result));
			PlasticSourceControlUtils::AddShelvedFileToChangelist(InOutChangelistsState, MoveTemp(AbsoluteFilename), ShelveState, MoveTemp(MovedFrom));
		}
		else
		{
			bResult = false;
		}
	}

	return bResult;
}

/**
 * Parse results of the 'cm find "shelves where owner='me'" --xml --encoding="utf-8"' command.
 *
 * Find shelves with comments starting like "ChangelistXXX: " and matching an existing Changelist number XXX
 *
 * Results of the find command looks like the following; note the "Changelist67: " prefix of the comment:
<?xml version="1.0" encoding="utf-8" ?>
<PLASTICQUERY>
  <SHELVE>
	<ID>1376</ID>
	<SHELVEID>9</SHELVEID>
    <COMMENT>Changelist67: test by Sebastien</COMMENT>
	<DATE>2022-06-30T16:39:55+02:00</DATE>
    <OWNER>sebastien.rombauts@unity3d.com</OWNER>
    <REPOSITORY>UE5PlasticPluginDev</REPOSITORY>
    <REPNAME>UE5PlasticPluginDev</REPNAME>
    <REPSERVER>test@cloud</REPSERVER>
	<PARENT>45</PARENT>
	<GUID>8fbefbcc-81a7-4b81-9b99-b51f4873d09f</GUID>
  </SHELVE>
  [...]
</PLASTICQUERY>
*/
static bool ParseShelvesResults(const FXmlFile& InXmlResult, TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates)
{
	static const FString PlasticQuery(TEXT("PLASTICQUERY"));
	static const FString Shelve(TEXT("SHELVE"));
	static const FString ShelveId(TEXT("SHELVEID"));
	static const FString Date(TEXT("DATE"));
	static const FString Comment(TEXT("COMMENT"));

	const FXmlNode* PlasticQueryNode = InXmlResult.GetRootNode();
	if (PlasticQueryNode == nullptr || PlasticQueryNode->GetTag() != PlasticQuery)
	{
		return false;
	}

	const TArray<FXmlNode*>& ShelvesNodes = PlasticQueryNode->GetChildrenNodes();
	for (const FXmlNode* ShelveNode : ShelvesNodes)
	{
		check(ShelveNode);
		const FXmlNode* ShelveIdNode = ShelveNode->FindChildNode(ShelveId);
		const FXmlNode* CommentNode = ShelveNode->FindChildNode(Comment);
		if (ShelveIdNode == nullptr || CommentNode == nullptr)
		{
			continue;
		}

		const FString& ShelveIdString = ShelveIdNode->GetContent();
		const FString& CommentString = DecodeXmlEntities(CommentNode->GetContent());

		// Search if there is a changelist matching the shelve (that is, a shelve with a comment starting with "ChangelistXXX: ")
		for (FPlasticSourceControlChangelistState& ChangelistState : InOutChangelistsStates)
		{
			FPlasticSourceControlChangelistRef Changelist = StaticCastSharedRef<FPlasticSourceControlChangelist>(ChangelistState.GetChangelist());
			const FString ChangelistPrefix = FString::Printf(TEXT("Changelist%s: "), *Changelist->GetName());
			if (CommentString.StartsWith(ChangelistPrefix))
			{
				ChangelistState.ShelveId = FCString::Atoi(*ShelveIdString);

				if (const FXmlNode* DateNode = ShelveNode->FindChildNode(Date))
				{
					const FString& DateIso = DateNode->GetContent();
					FDateTime::ParseIso8601(*DateIso, ChangelistState.ShelveDate);
				}
			}
		}
	}

	return true;
}


bool ParseShelvesResults(const FString& InResults, TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseShelvesResults::LoadXml);
		bResult = XmlFile.LoadFile(InResults, EConstructMethod::ConstructFromBuffer);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseShelvesResults::ParseXml);
		bResult = ParseShelvesResults(XmlFile, InOutChangelistsStates);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseShelvesResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

/**
 * Parse results of the 'cm diff sh:<ShelveId> --format="{status};{baserevid};{path}"' command.
 *
 * Results of the diff command looks like that:
C;666;Content\NewFolder\BP_CheckedOut.uasset
 * but for Moved assets there are two entries that we need to merge:
C;266;"Content\ThirdPerson\Blueprints\BP_ThirdPersonCharacterRenamed.uasset"
M;-1;"Content\ThirdPerson\Blueprints\BP_ThirdPersonCharacterRenamed.uasset"
*/
bool ParseShelveDiffResults(const FString InWorkspaceRoot, TArray<FString>&& InResults, TArray<FPlasticSourceControlRevision>& OutBaseRevisions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseShelveDiffResults);

	bool bResult = true;

	OutBaseRevisions.Reset(InResults.Num());
	for (FString& InResult : InResults)
	{
		TArray<FString> ResultElements;
		ResultElements.Reserve(3);
		InResult.ParseIntoArray(ResultElements, FILE_STATUS_SEPARATOR, false); // Don't cull empty values in csv
		if (ResultElements.Num() == 3 && ResultElements[0].Len() == 1)
		{
			EWorkspaceState ShelveState = ParseShelveFileStatus(ResultElements[0][0]);
			const int32 BaseRevisionId = FCString::Atoi(*ResultElements[1]);
			// Remove outer double quotes on filename
			FString File = MoveTemp(ResultElements[2]);
			File.MidInline(1, File.Len() - 2, EAllowShrinking::No);
			FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InWorkspaceRoot, File);

			if (ShelveState == EWorkspaceState::Moved)
			{
				// Note: in case of a Moved file, it appears twice in the list; just update the first entry (set as a "Changed") with the "Move" status
				if (FPlasticSourceControlRevision* ExistingShelveRevision = OutBaseRevisions.FindByPredicate(
					[&AbsoluteFilename](const FPlasticSourceControlRevision& State)
					{
						return State.GetFilename().Equals(AbsoluteFilename);
					}))
				{
					ExistingShelveRevision->Action = SourceControlActionMoved;
					continue;
				}
			}

			FPlasticSourceControlRevision SourceControlRevision;
			SourceControlRevision.Filename = MoveTemp(AbsoluteFilename);
			SourceControlRevision.Action = FileStateToAction(ShelveState);
			SourceControlRevision.RevisionId = BaseRevisionId;
			OutBaseRevisions.Add(MoveTemp(SourceControlRevision));
		}
		else
		{
			bResult = false;
		}
	}

	return bResult;
}

/**
 * Parse results of the 'cm find "shelves where ShelveId='NNN'" --xml --encoding="utf-8"' command.
 *
 * Results of the find command looks like the following; note the "Changelist67: " prefix of the comment:
<?xml version="1.0" encoding="utf-8" ?>
<PLASTICQUERY>
  <SHELVE>
	<ID>1376</ID>
	<SHELVEID>9</SHELVEID>
	<COMMENT>Changelist67: test by Sebastien</COMMENT>
	<DATE>2022-06-30T16:39:55+02:00</DATE>
	<OWNER>sebastien.rombauts@unity3d.com</OWNER>
	<REPOSITORY>UE5PlasticPluginDev</REPOSITORY>
	<REPNAME>UE5PlasticPluginDev</REPNAME>
	<REPSERVER>test@cloud</REPSERVER>
	<PARENT>45</PARENT>
	<GUID>8fbefbcc-81a7-4b81-9b99-b51f4873d09f</GUID>
  </SHELVE>
  [...]
</PLASTICQUERY>
*/
static bool ParseShelvesResult(const FXmlFile& InXmlResult, int32& OutShelveId, FString& OutComment, FDateTime& OutDate, FString& OutOwner)
{
	static const FString PlasticQuery(TEXT("PLASTICQUERY"));
	static const FString Shelve(TEXT("SHELVE"));
	static const FString ShelveId(TEXT("SHELVEID"));
	static const FString Comment(TEXT("COMMENT"));
	static const FString Date(TEXT("DATE"));

	const FXmlNode* PlasticQueryNode = InXmlResult.GetRootNode();
	if (PlasticQueryNode == nullptr || PlasticQueryNode->GetTag() != PlasticQuery)
	{
		return false;
	}

	const TArray<FXmlNode*>& ShelvesNodes = PlasticQueryNode->GetChildrenNodes();
	if (ShelvesNodes.Num() < 1)
	{
		return false;
	}

	if (const FXmlNode* ShelveNode = ShelvesNodes[0])
	{
		check(ShelveNode);
		if (const FXmlNode* ShelveIdNode = ShelveNode->FindChildNode(ShelveId))
		{
			OutShelveId = FCString::Atoi(*ShelveIdNode->GetContent());
		}
		if (const FXmlNode* CommentNode = ShelveNode->FindChildNode(Comment))
		{
			OutComment = DecodeXmlEntities(CommentNode->GetContent());
		}
		if (const FXmlNode* DateNode = ShelveNode->FindChildNode(Date))
		{
			FDateTime::ParseIso8601(*DateNode->GetContent(), OutDate);
		}
	}

	return true;
}

bool ParseShelvesResult(const FString& InResults, FString& OutComment, FDateTime& OutDate, FString& OutOwner)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseShelvesResult::LoadXml);
		bResult = XmlFile.LoadFile(InResults, EConstructMethod::ConstructFromBuffer);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseShelvesResult::ParseXml);
		int32 ShelveId;
		bResult = PlasticSourceControlParsers::ParseShelvesResult(XmlFile, ShelveId, OutComment, OutDate, OutOwner);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseShelvesResult: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}


/**
 * Parse results of the 'cm find changeset --xml --encoding="utf-8"' command.
 *
 * Results of the find command looks like the following:
<?xml version="1.0" encoding="utf-8" ?>
<PLASTICQUERY>
  <CHANGESET>
    <ID>2652</ID>
    <CHANGESETID>56</CHANGESETID>
    <COMMENT>test</COMMENT>
    <DATE>2024-03-25T10:37:14+01:00</DATE>
    <OWNER>sebastien.rombauts@unity3d.com</OWNER>
    <REPOSITORY>UE5PlasticPluginDev</REPOSITORY>
    <REPNAME>UE5PlasticPluginDev</REPNAME>
    <REPSERVER>SRombautsU@cloud</REPSERVER>
    <BRANCH>/main</BRANCH>
    <PARENT>55</PARENT>
    <GUID>d49c552e-9654-44d0-86eb-0d55fa5e8dc3</GUID>
    <ROOTREV>2651</ROOTREV>
  </CHANGESET>
  [...]
</PLASTICQUERY>
*/
static bool ParseChangesetesResults(const FXmlFile& InXmlResult, TArray<FPlasticSourceControlChangesetRef>& OutChangesets)
{
	static const FString PlasticQuery(TEXT("PLASTICQUERY"));
	static const FString ChangesetId(TEXT("CHANGESETID"));
	static const FString Branch(TEXT("BRANCH"));
	static const FString Comment(TEXT("COMMENT"));
	static const FString Owner(TEXT("OWNER"));
	static const FString Date(TEXT("DATE"));

	const FXmlNode* PlasticQueryNode = InXmlResult.GetRootNode();
	if (PlasticQueryNode == nullptr || PlasticQueryNode->GetTag() != PlasticQuery)
	{
		return false;
	}

	const TArray<FXmlNode*>& ChangesetsNodes = PlasticQueryNode->GetChildrenNodes();
	OutChangesets.Reserve(ChangesetsNodes.Num());
	for (const FXmlNode* ChangesetNode : ChangesetsNodes)
	{
		check(ChangesetNode);
		const FXmlNode* ChangesetIdNode = ChangesetNode->FindChildNode(ChangesetId);
		if (ChangesetIdNode == nullptr)
		{
			continue;
		}

		FPlasticSourceControlChangesetRef ChangesetRef = MakeShareable(new FPlasticSourceControlChangeset());

		ChangesetRef->ChangesetId = FCString::Atoi(*ChangesetIdNode->GetContent());

		if (const FXmlNode* CommentNode = ChangesetNode->FindChildNode(Comment))
		{
			ChangesetRef->Comment = DecodeXmlEntities(CommentNode->GetContent());
		}
		if (const FXmlNode* BranchNode = ChangesetNode->FindChildNode(Branch))
		{
			ChangesetRef->Branch = DecodeXmlEntities(BranchNode->GetContent());
		}
		if (const FXmlNode* OwnerNode = ChangesetNode->FindChildNode(Owner))
		{
			// Note: keeping the full email address as the owner name so we can display both the short and full name in the tooltip
			ChangesetRef->CreatedBy = OwnerNode->GetContent();
		}
		if (const FXmlNode* DateNode = ChangesetNode->FindChildNode(Date))
		{
			const FString& DateIso = DateNode->GetContent();
			FDateTime::ParseIso8601(*DateIso, ChangesetRef->Date);
		}

		OutChangesets.Add(MoveTemp(ChangesetRef));
	}

	return true;
}

bool ParseChangesetsResults(const FString& InXmlFilename, TArray<FPlasticSourceControlChangesetRef>& OutChangesets)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseChangesetesResults::LoadXml);
		bResult = XmlFile.LoadFile(InXmlFilename);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseChangesetesResults::ParseXml);
		bResult = ParseChangesetesResults(XmlFile, OutChangesets);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseChangesetesResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}


/**
 * Convert the type of change in the log of changesets to a state.
 */
static EWorkspaceState StateFromType(const FString& InChangeType)
{
	static const FString Added(TEXT("Added"));
	static const FString Changed(TEXT("Changed"));
	static const FString Deleted(TEXT("Deleted"));
	static const FString Moved(TEXT("Moved"));

	if (InChangeType.Equals(Changed))
	{
		return EWorkspaceState::CheckedOutChanged;
	}
	else if (InChangeType.Equals(Added))
	{
		return EWorkspaceState::Added;
	}
	else if (InChangeType.Equals(Moved))
	{
		return EWorkspaceState::Moved;
	}
	else if (InChangeType.Equals(Deleted))
	{
		return EWorkspaceState::Deleted;
	}

	return EWorkspaceState::Unknown;
}

/**
 * Parse Changes child node in a Changeset.
 *
 * Results of the log command looks like the following:
<?xml version="1.0" encoding="utf-8"?>
<LogList>
  <Changeset>
	[...]
	<Changes>
	  <Item>
		<Branch>/main/test</Branch>
		<RevNo>72</RevNo>
		<Owner>sebastien.rombauts@unity3d.com</Owner>
		<RevId>2861</RevId>
		<ParentRevId>-1</ParentRevId>
		<SrcCmPath>/Private/Private.md</SrcCmPath>
		<SrcParentItemId>2868</SrcParentItemId>
		<DstCmPath>/Private/Private.md</DstCmPath>
		<DstParentItemId>2868</DstParentItemId>
		<Date>2024-04-03T14:59:31+02:00</Date>
		<Type>Added</Type>
	  </Item>
	[...]
	</Changes>
	[...]
  </Changeset>
</LogList>
 */
static void ParseChangesInChangeset(const FXmlNode* InChangesetNode, const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles)
{
	static const FString Changes(TEXT("Changes"));
	static const FString Item(TEXT("Item"));
	static const FString Type(TEXT("Type"));
	static const FString SrcCmPath(TEXT("SrcCmPath"));
	static const FString DstCmPath(TEXT("DstCmPath"));

	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const FString RootRepSpec = FString::Printf(TEXT("%s@%s"), *Provider.GetRepositoryName(), *Provider.GetServerUrl());

	if (const FXmlNode* ChangesNode = InChangesetNode->FindChildNode(Changes))
	{
		const TArray<FXmlNode*>& ItemNodes = ChangesNode->GetChildrenNodes();
		OutFiles.Reserve(ItemNodes.Num());
		for (const FXmlNode* ItemNode : ItemNodes)
		{
			check(ItemNode);

			const FXmlNode* PathNode = ItemNode->FindChildNode(DstCmPath);
			const FXmlNode* TypeNode = ItemNode->FindChildNode(Type);
			if ((PathNode == nullptr) || (TypeNode == nullptr))
			{
				continue;
			}

			// Note: remove the leading '/' from the server path to make it relative to the root of the workspace
			FString FileName = PathNode->GetContent().RightChop(1);
			const EWorkspaceState WorkspaceState = StateFromType(TypeNode->GetContent());
			FPlasticSourceControlStateRef State = MakeShareable(new FPlasticSourceControlState(MoveTemp(FileName), WorkspaceState));
			State->RepSpec = RootRepSpec;

			if (WorkspaceState == EWorkspaceState::Moved)
			{
				if (const FXmlNode* SrcNode = ItemNode->FindChildNode(SrcCmPath))
				{
					State->MovedFrom = SrcNode->GetContent().RightChop(1); // remove the leading '/' character from the server path
				}
			}

			// Add one revision to be able to fetch the file content for diff, if it's not marked for deletion.
			if ((WorkspaceState != EWorkspaceState::Deleted) && (State->History.Num() == 0))
			{
				const TSharedRef<FPlasticSourceControlRevision, ESPMode::ThreadSafe> SourceControlRevision = MakeShareable(new FPlasticSourceControlRevision);
				SourceControlRevision->State = &State.Get();
				SourceControlRevision->Filename = State->GetFilename();
				SourceControlRevision->Revision = FString::Printf(TEXT("cs:%d"), InChangeset->ChangesetId);
				SourceControlRevision->ChangesetNumber = InChangeset->ChangesetId; // Note: for display in the diff window only
				SourceControlRevision->Date = InChangeset->Date; // Note: not yet used for display as of UE5.2

				State->History.Add(SourceControlRevision);
			}

			// Note: in case of a Moved file, it appears twice in the list; just update the first entry (set as a "Changed") with the "Move" status
			if (FPlasticSourceControlStateRef* ExistingState = OutFiles.FindByPredicate(
				[&State](const TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe>& InState)
				{
					return InState->GetFilename().Equals(State->GetFilename());
				}))
			{
				(*ExistingState)->WorkspaceState = State->WorkspaceState;
				(*ExistingState)->MovedFrom = State->MovedFrom;
			}
			else
			{
				OutFiles.Add(MoveTemp(State));
			}
		}
	}
}

/**
 * Parse results of the 'cm log cs:<ChangesetId> --xml --encoding="utf-8"' command.
 *
 * Results of the log command looks like the following:
<?xml version="1.0" encoding="utf-8"?>
<LogList>
  <Changeset>
	<ObjId>2674</ObjId>
	<ChangesetId>73</ChangesetId>
	<Branch>/main/test</Branch>
	<Comment>private files and folders</Comment>
	<Owner>sebastien.rombauts@unity3d.com</Owner>
	<GUID>cd803bd1-7d59-4573-b9de-1a4e684d573a</GUID>
	<Changes>
	  <Item>
		<Branch>/main/test</Branch>
		<RevNo>72</RevNo>
		<Owner>sebastien.rombauts@unity3d.com</Owner>
		<RevId>2861</RevId>
		<ParentRevId>-1</ParentRevId>
		<SrcCmPath>/Private/Private.md</SrcCmPath>
		<SrcParentItemId>2868</SrcParentItemId>
		<DstCmPath>/Private/Private.md</DstCmPath>
		<DstParentItemId>2868</DstParentItemId>
		<Date>2024-04-03T14:59:31+02:00</Date>
		<Type>Added</Type>
	  </Item>
	[...]
	</Changes>
	<Date>2024-04-02T16:20:11+02:00</Date>
  </Changeset>
</LogList>
*/
static bool ParseLogResults(const FXmlFile& InXmlResult, const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles)
{
	static const FString LogList(TEXT("LogList"));
	static const FString ChangesetId(TEXT("ChangesetId"));
	static const FString Branch(TEXT("Branch"));
	static const FString Comment(TEXT("Comment"));
	static const FString Owner(TEXT("Owner"));
	static const FString Date(TEXT("Date"));

	const FXmlNode* LogListNode = InXmlResult.GetRootNode();
	if (LogListNode == nullptr || LogListNode->GetTag() != LogList)
	{
		return false;
	}

	const TArray<FXmlNode*>& ChangesetsNodes = LogListNode->GetChildrenNodes();
	if (ChangesetsNodes.Num() != 1 || ChangesetsNodes[0] == nullptr)
	{
		return false;
	}

	FXmlNode* ChangesetNode = ChangesetsNodes[0];
	const FXmlNode* ChangesetIdNode = ChangesetNode->FindChildNode(ChangesetId);
	if (ChangesetIdNode == nullptr || InChangeset->ChangesetId != FCString::Atoi(*ChangesetIdNode->GetContent()))
	{
		return false;
	}

	// List Files States and create a Revision
	ParseChangesInChangeset(ChangesetNode, InChangeset, OutFiles);

	return true;
}

bool ParseLogResults(const FString& InXmlFilename, const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseLogResults::LoadXml);
		bResult = XmlFile.LoadFile(InXmlFilename);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseLogResults::ParseXml);
		bResult = ParseLogResults(XmlFile, InChangeset, OutFiles);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseLogResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

/**
 * Parse results of the 'cm find "branches where date >= 'YYYY-MM-DD' or changesets >= 'YYYY-MM-DD'" --xml --encoding="utf-8"' command.
 *
 * Results of the find command looks like the following:
<?xml version="1.0" encoding="utf-8" ?>
<PLASTICQUERY>
  <BRANCH>
	<ID>3</ID>
	<COMMENT>main branch</COMMENT>
	<DATE>2023-10-18T15:08:49+02:00</DATE>
	<OWNER>sebastien.rombauts@unity3d.com</OWNER>
	<NAME>/main</NAME>
	<PARENT></PARENT>
	<REPOSITORY>UE5PlasticPluginDev</REPOSITORY>
	<REPNAME>UE5PlasticPluginDev</REPNAME>
	<REPSERVER>SRombautsU@cloud</REPSERVER>
	<TYPE>T</TYPE>
	<CHANGESET>4</CHANGESET>
	<GUID>5fc2d7c8-05e1-4987-9dd9-74eaec7c27eb</GUID>
  </BRANCH>
  [...]
</PLASTICQUERY>
*/
static bool ParseBranchesResults(const FXmlFile& InXmlResult, TArray<FPlasticSourceControlBranchRef>& OutBranches)
{
	static const FString PlasticQuery(TEXT("PLASTICQUERY"));
	static const FString Branch(TEXT("BRANCH"));
	static const FString Comment(TEXT("COMMENT"));
	static const FString Date(TEXT("DATE"));
	static const FString Owner(TEXT("OWNER"));
	static const FString Name(TEXT("NAME"));
	static const FString RepName(TEXT("REPNAME"));
	static const FString RepServer(TEXT("REPSERVER"));

	const FXmlNode* PlasticQueryNode = InXmlResult.GetRootNode();
	if (PlasticQueryNode == nullptr || PlasticQueryNode->GetTag() != PlasticQuery)
	{
		return false;
	}

	const TArray<FXmlNode*>& BranchesNodes = PlasticQueryNode->GetChildrenNodes();
	OutBranches.Reserve(BranchesNodes.Num());
	for (const FXmlNode* BranchNode : BranchesNodes)
	{
		check(BranchNode);
		const FXmlNode* NameNode = BranchNode->FindChildNode(Name);
		if (NameNode == nullptr)
		{
			continue;
		}

		FPlasticSourceControlBranchRef BranchRef = MakeShareable(new FPlasticSourceControlBranch());

		BranchRef->Name = DecodeXmlEntities(NameNode->GetContent());

		if (const FXmlNode* CommentNode = BranchNode->FindChildNode(Comment))
		{
			BranchRef->Comment = DecodeXmlEntities(CommentNode->GetContent());
		}
		if (const FXmlNode* DateNode = BranchNode->FindChildNode(Date))
		{
			const FString& DateIso = DateNode->GetContent();
			FDateTime::ParseIso8601(*DateIso, BranchRef->Date);
		}
		if (const FXmlNode* OwnerNode = BranchNode->FindChildNode(Owner))
		{
			// Note: keeping the full email address as the owner name so we can display both the short and full name in the tooltip
			BranchRef->CreatedBy = OwnerNode->GetContent();
		}
		if (const FXmlNode* RepNameNode = BranchNode->FindChildNode(RepName))
		{
			if (const FXmlNode* RepServerNode = BranchNode->FindChildNode(RepServer))
			{
				BranchRef->Repository = RepNameNode->GetContent() + TEXT("@") + RepServerNode->GetContent();
			}
		}

		OutBranches.Add(MoveTemp(BranchRef));
	}

	return true;
}

bool ParseBranchesResults(const FString& InXmlFilename, TArray<FPlasticSourceControlBranchRef>& OutBranches)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseBranchesResults::LoadXml);
		bResult = XmlFile.LoadFile(InXmlFilename);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseBranchesResults::ParseXml);
		bResult = ParseBranchesResults(XmlFile, OutBranches);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseBranchesResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

/* Parse results of the 'cm merge --xml=tempfile.xml --encoding="utf-8" --merge <branch-name>' command.
 *
 * Results of the merge command looks like that:
<Merge>
  <Added />
  <Deleted />
  <Changed>
	<MergeItem>
	  <ItemType>File</ItemType>
	  <Path>/Content/ThirdPerson/Blueprints/BP_Cube.uasset</Path>
	  <Size>19730</Size>
	  <User>sebastien.rombauts@unity3d.com</User>
	  <Date>2023-11-21T13:35:04+01:00</Date>
	</MergeItem>
  </Changed>
  <Moved />
  <PermissionsChanged />
  <Warnings />
  <DirConflicts />
  <FileConflicts />
</Merge>
*/
static bool ParseMergeResults(const FXmlFile& InXmlResult, TArray<FString>& OutFiles)
{
	static const FString Merge(TEXT("Merge"));
	static const FString Added(TEXT("Added"));
	static const FString Deleted(TEXT("Deleted"));
	static const FString Changed(TEXT("Changed"));
	static const FString Moved(TEXT("Moved"));
	static const TArray<FString> MergeTypes({ Added, Deleted, Changed, Moved });
	static const FString MergedItem(TEXT("MergeItem"));
	static const FString Path(TEXT("Path"));
	static const FString DstPath(TEXT("DstPath"));

	const FString WorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot().LeftChop(1);

	const FXmlNode* MergeNode = InXmlResult.GetRootNode();
	if (MergeNode == nullptr || MergeNode->GetTag() != Merge)
	{
		return false;
	}

	for (const FString& MergeType : MergeTypes)
	{
		if (const FXmlNode* MergeTypeNode = MergeNode->FindChildNode(MergeType))
		{
			const TArray<FXmlNode*>& MergeItemNodes = MergeTypeNode->GetChildrenNodes();
			for (const FXmlNode* MergeItemNode : MergeItemNodes)
			{
				const FString PathName = MergeType == Moved ? DstPath : Path;
				if (const FXmlNode* PathNode = MergeItemNode->FindChildNode(PathName))
				{
					FString Filename = FPaths::Combine(WorkspaceRoot, PathNode->GetContent());
					FPaths::NormalizeFilename(Filename);
					if (!OutFiles.Contains(Filename))
					{
						OutFiles.Add(Filename);
					}
				}
			}
		}
	}

	return true;
}

bool ParseMergeResults(const FString& InResult, TArray<FString>& OutFiles)
{
	bool bResult = false;

	FXmlFile XmlFile;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseMergeResults::LoadXml);
		bResult = XmlFile.LoadFile(InResult, EConstructMethod::ConstructFromBuffer);
	}
	if (bResult)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlParsers::ParseMergeResults::ParseXml);
		bResult = ParseMergeResults(XmlFile, OutFiles);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "ParseMergeResults: XML parse error '%ls'", *XmlFile.GetLastError())
	}

	return bResult;
}

} // namespace PlasticSourceControlParsers
