// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPlasticSourceControlChangelistState;
class FPlasticSourceControlLock;
class FPlasticSourceControlRevision;
class FPlasticSourceControlState;
typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;
typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStateRef;

namespace PlasticSourceControlParsers
{

FPlasticSourceControlLock ParseLockInfo(const FString& InResult);

class FPlasticMergeConflictParser
{
public:
	explicit FPlasticMergeConflictParser(const FString& InResult);

	FString Filename;
	FString BaseChangeset;
	FString SourceChangeset;
};

/**
	* Helper struct for RemoveRedundantErrors()
	*/
struct FRemoveRedundantErrors
{
	explicit FRemoveRedundantErrors(const FString& InFilter)
		: Filter(InFilter)
	{
	}

	bool operator()(const FString& String) const
	{
		if (String.Contains(Filter))
		{
			return true;
		}

		return false;
	}

	/** The filter string we try to identify in the reported error */
	FString Filter;
};

bool ParseProfileInfo(TArray<FString>& InResults, const FString& InServerUrl, FString& OutUserName);

bool ParseWorkspaceInfo(TArray<FString>& InResults, FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl);

bool GetChangesetFromWorkspaceStatus(const TArray<FString>& InResults, int32& OutChangeset);

void ParseFileStatusResult(TArray<FString>&& InFiles, const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& OutStates);

void ParseDirectoryStatusResult(const FString& InDir, const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& OutStates);

void ParseFileinfoResults(const TArray<FString>& InResults, TArray<FPlasticSourceControlState>& InOutStates);

bool ParseHistoryResults(const bool bInUpdateHistory, const FString& InXmlFilename, TArray<FPlasticSourceControlState>& InOutStates);

bool ParseUpdateResults(const FString& InResults, TArray<FString>& OutFiles);
bool ParseUpdateResults(const TArray<FString>& InResults, TArray<FString>& OutFiles);

FText ParseCheckInResults(const TArray<FString>& InResults);

bool ParseChangelistsResults(const FString& InXmlFilename, TArray<FPlasticSourceControlChangelistState>& OutChangelistsStates, TArray<TArray<FPlasticSourceControlState>>& OutCLFilesStates);

bool ParseShelveDiffResult(const FString InWorkspaceRoot, TArray<FString>&& InResults, FPlasticSourceControlChangelistState& InOutChangelistsState);
bool ParseShelveDiffResults(const FString InWorkspaceRoot, TArray<FString>&& InResults, TArray<FPlasticSourceControlRevision>& OutBaseRevisions);

bool ParseShelvesResults(const FString& InResults, TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates);
bool ParseShelvesResult(const FString& InResults, FString& OutComment, FDateTime& OutDate, FString& OutOwner);

bool ParseChangesetsResults(const FString& InXmlFilename, TArray<FPlasticSourceControlChangesetRef>& OutChangesets);
bool ParseLogResults(const FString& InXmlFilename, const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles);

bool ParseBranchesResults(const FString& InXmlFilename, TArray<FPlasticSourceControlBranchRef>& OutBranches);

bool ParseMergeResults(const FString& InResult, TArray<FString>& OutFiles);

} // namespace PlasticSourceControlParsers
