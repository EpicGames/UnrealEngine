// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "PlasticSourceControlProjectSettings.generated.h"

/** Project Settings for Unity Version Control (formerly Plastic SCM). Saved in Config/DefaultEditor.ini */
UCLASS(config = Editor, defaultconfig, meta = (DisplayName = "Source Control - Unity Version Control"))
class UPlasticSourceControlProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Map Unity Version Control user names (typically e-mail addresses or company domain names) to display names for brevity. */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control")
	TMap<FString, FString> UserNameToDisplayName;

	/** Hide the domain part of an username e-mail address (eg @gmail.com) if the UserNameToDisplayName map didn't match (enabled by default). */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control")
	bool bHideEmailDomainInUsername = true;

	/** If enabled, you'll be prompted to check out changed files (enabled by default). Checkout is needed to work with Changelists. */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control")
	bool bPromptForCheckoutOnChange = true;

	/** If a non-null value is set, limit the maximum number of revisions requested to Unity Version Control to display in the "History" window. */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control", meta = (ClampMin = 0))
	int32 LimitNumberOfRevisionsInHistory = 50;

	/** Show the repository where the branch is created (hidden by default) */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Branches window")
	bool bShowBranchRepositoryColumn = false;

	/* Show the name of the creator of the branch */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Branches window")
	bool bShowBranchCreatedByColumn = true;

	/* Show the date of creation of the branch */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Branches window")
	bool bShowBranchDateColumn = true;

	/* Show the comment of the branch */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Branches window")
	bool bShowBranchCommentColumn = true;

	/* Show the Id of the lock (hidden by default) */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Locks window")
	bool bShowLockIdColumn = false;

	/* Show the Workspace of the lock */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Locks window")
	bool bShowLockWorkspaceColumn = true;

	/* Show the Date of the lock */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Locks window")
	bool bShowLockDateColumn = true;

	/* Show the Destination Branch of the lock (hidden by default) */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Locks window")
	bool bShowLockDestinationBranchColumn = false;

	/* Show the name of the creator of the changeset */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Changesets window")
	bool bShowChangesetCreatedByColumn = true;

	/* Show the date of creation of the changeset */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Changesets window")
	bool bShowChangesetDateColumn = true;

	/* Show the comment of the changeset */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Changesets window")
	bool bShowChangesetCommentColumn = true;

	/* Show the branch the changeset was created on */
	UPROPERTY(config, EditAnywhere, Category = "Unity Version Control|View Changesets window")
	bool bShowChangesetBranchColumn = true;
};
