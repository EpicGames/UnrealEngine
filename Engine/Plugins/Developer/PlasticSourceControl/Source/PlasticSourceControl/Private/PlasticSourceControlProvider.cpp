// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlProvider.h"

#include "PlasticSourceControlChangelistState.h"
#include "PlasticSourceControlCommand.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlProjectSettings.h"
#include "PlasticSourceControlSettings.h"
#include "PlasticSourceControlShell.h"
#include "PlasticSourceControlState.h"
#include "PlasticSourceControlUtils.h"
#include "PlasticSourceControlVersions.h"
#include "SPlasticSourceControlSettings.h"

#include "ISourceControlModule.h"
#include "Logging/MessageLog.h"
#include "ScopedSourceControlProgress.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Interfaces/IPluginManager.h"

#include "Algo/Transform.h"
#include "Misc/Paths.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformProcess.h"
#include "Misc/QueuedThreadPool.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/SavePackage.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

static FName ProviderName("Plastic SCM");

FPlasticSourceControlProvider::FPlasticSourceControlProvider()
{
	PlasticSourceControlSettings.LoadSettings();

	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FPlasticSourceControlProvider::HandlePackageSaved);
}

FPlasticSourceControlProvider::~FPlasticSourceControlProvider()
{
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
}

void FPlasticSourceControlProvider::Init(bool bForceConnection)
{
	// Init() is called multiple times at startup: do not check Unity Version Control each time
	if (!bPlasticAvailable)
	{
		const TSharedPtr<IPlugin> Plugin = FPlasticSourceControlModule::GetPlugin();
		if (Plugin.IsValid())
		{
			PluginVersion = Plugin->GetDescriptor().VersionName;
			UE_LOGF(LogSourceControl, Log, "Unity Version Control (formerly Plastic SCM) plugin %ls", *PluginVersion);
		}

		CheckPlasticAvailability();

		FMessageLog("SourceControl").Info(FText::Format(LOCTEXT("PluginVersion", "Unity Version Control (formerly Plastic SCM) {0} (plugin {1})"),
			FText::FromString(PlasticScmVersion.String), FText::FromString(PluginVersion)));

		// Override the source control logs verbosity level if needed based on settings
		if (AccessSettings().GetEnableVerboseLogs())
		{
			PlasticSourceControlUtils::SwitchVerboseLogs(true);
		}
	}

	if (bForceConnection && bPlasticAvailable && bWorkspaceFound && !bServerAvailable)
	{
		TArray<FString> InfoMessages, ErrorMessages;
		TArray<FString> Parameters;
		// Execute a 'checkconnection' command to set bServerAvailable based on the connectivity of the server
		bServerAvailable = PlasticSourceControlUtils::RunCheckConnection(WorkspaceSelector, BranchName, RepositoryName, ServerUrl, InfoMessages, ErrorMessages);
		if (!bServerAvailable)
		{
			FMessageLog SourceControlLog("SourceControl");
			for (const FString& ErrorMessage : ErrorMessages)
			{
				SourceControlLog.Error(FText::FromString(ErrorMessage));
			}
		}
	}
}

void FPlasticSourceControlProvider::CheckPlasticAvailability()
{
	FString PathToPlasticBinary = AccessSettings().GetBinaryPath();
	if (PathToPlasticBinary.IsEmpty())
	{
		bPlasticAvailable = false;

		// Try to find cm binary, and update settings accordingly
		PathToPlasticBinary = PlasticSourceControlUtils::FindPlasticBinaryPath();
		if (!PathToPlasticBinary.IsEmpty())
		{
			AccessSettings().SetBinaryPath(PathToPlasticBinary);
		}
	}

	if (!PathToPlasticBinary.IsEmpty())
	{
		const FString PathToProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

		// Launch the Unity Version Control cli shell on the background to issue all commands during this session
		bPlasticAvailable = PlasticSourceControlShell::Launch(PathToPlasticBinary, PathToProjectDir);
		if (!bPlasticAvailable)
		{
			return;
		}

		bPlasticAvailable = PlasticSourceControlUtils::GetPlasticScmVersion(PlasticScmVersion);
		if (!bPlasticAvailable)
		{
			return;
		}

		FString ActualPathToPlasticBinary;
		PlasticSourceControlUtils::GetCmLocation(ActualPathToPlasticBinary);

		// Find the path to the root Plastic directory (if any, else uses the ProjectDir)
		bWorkspaceFound = PlasticSourceControlUtils::GetWorkspacePath(PathToProjectDir, PathToWorkspaceRoot);

		bUsesLocalReadOnlyState = PlasticSourceControlUtils::GetConfigSetFilesAsReadOnly();

		// Register Console Commands
		PlasticSourceControlConsole.Register();

		if (bWorkspaceFound)
		{
			TArray<FString> ErrorMessages;
			PlasticSourceControlUtils::GetWorkspaceInfo(WorkspaceSelector, BranchName, RepositoryName, ServerUrl, ErrorMessages);
			UserName = PlasticSourceControlUtils::GetProfileUserName(ServerUrl);
		}
		else
		{
			// This info message is only useful here, if bPlasticAvailable, for the Login window
			FFormatNamedArguments Args;
			Args.Add(TEXT("WorkspacePath"), FText::FromString(PathToWorkspaceRoot));
			FMessageLog("SourceControl").Info(FText::Format(LOCTEXT("NotInAWorkspace", "{WorkspacePath} is not in a workspace."), Args));

			// Get default server and user name (from the global client config)
			ServerUrl = PlasticSourceControlUtils::GetConfigDefaultRepServer();
			UserName = PlasticSourceControlUtils::GetDefaultUserName();
		}
	}
}

void FPlasticSourceControlProvider::Close()
{
	// clear the cache
	StateCache.Empty();
	// terminate the background 'cm shell' process and associated pipes
	PlasticSourceControlShell::Terminate();
	// Remove all extensions to the "Source Control" menu in the Editor Toolbar
	PlasticSourceControlMenu.Unregister();
	// Unregister Console Commands
	PlasticSourceControlConsole.Unregister();

	bServerAvailable = false;
	bPlasticAvailable = false;
	bWorkspaceFound = false;
	UserName.Empty();
}

TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlProvider::GetStateInternal(const FString& InFilename)
{
	TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe>* State = StateCache.Find(InFilename);
	if (State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		// cache an unknown state for this item
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> NewState = MakeShareable(new FPlasticSourceControlState(FString(InFilename)));
		StateCache.Add(InFilename, NewState);
		return NewState;
	}
}

TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> FPlasticSourceControlProvider::GetStateInternal(const FPlasticSourceControlChangelist& InChangelist)
{
	TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe>* State = ChangelistsStateCache.Find(InChangelist);
	if (State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		// cache an unknown state for this item
		TSharedRef<FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> NewState = MakeShared<FPlasticSourceControlChangelistState>(InChangelist);
		ChangelistsStateCache.Add(InChangelist, NewState);
		return NewState;
	}
}

// Note: called once for each asset being saved, which can be hundreds in the case of a map using One File Per Actor (OFPA) in UE5
void FPlasticSourceControlProvider::HandlePackageSaved(const FString& InPackageFilename, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(InPackageFilename);
	auto FileState = GetStateInternal(AbsoluteFilename);

	// Note: the Editor doesn't ask to refresh the source control status of an asset after it is saved, only *before* (to check that it's possible to save)
	// So when an asset with no change is saved, update its state in cache to record the fact that the asset is now changed.
	// Note that updating the state in cache isn't enough to refresh the status icon in the Content Browser or the View Changes windows (since the Editor isn't made aware of the change)
	// so don't trigger the "Changed" status here, it will be handled on the next update status operation.
	if (FileState->WorkspaceState == EWorkspaceState::CheckedOutUnchanged)
	{
		FileState->WorkspaceState = EWorkspaceState::CheckedOutChanged; // In this case the "CheckedOut" icon is already displayed (both states are using the same status icon)
	}
}

FText FPlasticSourceControlProvider::GetStatusText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("PlasticScmVersion"), FText::FromString(PlasticScmVersion.String));
	Args.Add(TEXT("PluginVersion"), FText::FromString(PluginVersion));
	Args.Add(TEXT("WorkspacePath"), FText::FromString(PathToWorkspaceRoot));
	Args.Add(TEXT("WorkspaceName"), FText::FromString(WorkspaceName));
	Args.Add(TEXT("WorkspaceSelector"), FText::FromString(WorkspaceSelector));
	Args.Add(TEXT("RepositoryName"), FText::FromString(RepositoryName));
	Args.Add(TEXT("ServerUrl"), FText::FromString(ServerUrl));
	// Detect special case for a partial checkout (CS:-1 in Gluon mode)!
	if (IsPartialWorkspace())
	{
		Args.Add(TEXT("ChangesetNumber"), FText::FromString(FString::Printf(TEXT("N/A  (Gluon partial workspace)"))));
	}
	else
	{
		Args.Add(TEXT("ChangesetNumber"), FText::FromString(FString::Printf(TEXT("%d  (regular full workspace)"), ChangesetNumber)));
	}
	Args.Add(TEXT("UserName"), FText::FromString(UserName));
	const FString DisplayName = PlasticSourceControlUtils::UserNameToDisplayName(UserName);
	if (DisplayName != UserName)
	{
		Args.Add(TEXT("DisplayName"), FText::FromString(TEXT("(Display: ") + DisplayName + TEXT(")")));
	}
	else
	{
		Args.Add(TEXT("DisplayName"), FText::GetEmpty());
	}

	FText FormattedError;
	TArray<FString> RecentErrors = GetLastErrors();
	if (RecentErrors.Num() > 0)
	{
		FFormatNamedArguments ErrorArgs;
		ErrorArgs.Add(TEXT("ErrorText"), FText::FromString(RecentErrors[0]));

		FormattedError = FText::Format(LOCTEXT("PlasticErrorStatusText", "Error: {ErrorText} {UserName}\n\n"), ErrorArgs);
	}
	Args.Add(TEXT("ErrorText"), FormattedError);

	return FText::Format(LOCTEXT("PlasticStatusText", "{ErrorText}Unity Version Control (formerly Plastic SCM) {PlasticScmVersion}\t(plugin v{PluginVersion})\nWorkspace: {WorkspaceName}  ({WorkspacePath})\n{WorkspaceSelector}@{RepositoryName}@{ServerUrl}\nChangeset: {ChangesetNumber}\nUser: '{UserName}'  {DisplayName}"), Args);
}

TMap<ISourceControlProvider::EStatus, FString> FPlasticSourceControlProvider::GetStatus() const
{
	TMap<EStatus, FString> Result;
	Result.Add(EStatus::Enabled, IsEnabled() ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::Connected, (IsEnabled() && IsAvailable()) ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::User, UserName);

	Result.Add(EStatus::ScmVersion, PlasticScmVersion.String);
	Result.Add(EStatus::PluginVersion, PluginVersion);
	Result.Add(EStatus::WorkspacePath, PathToWorkspaceRoot);
	Result.Add(EStatus::Workspace, WorkspaceName);
	Result.Add(EStatus::Branch, WorkspaceSelector);
	if (!IsPartialWorkspace())
	{
		Result.Add(EStatus::Changeset, FString::Printf(TEXT("%d"), ChangesetNumber));
	}
	return Result;
}

/** Quick check if source control is enabled. Specifically, it returns true if a source control provider is set (regardless of whether the provider is available) and false if no provider is set. So all providers except the stub DefaultSourceProvider will return true. */
bool FPlasticSourceControlProvider::IsEnabled() const
{
	return true;
}

/** Quick check if source control is available for use (return whether the server is available or not) */
bool FPlasticSourceControlProvider::IsAvailable() const
{
	return bServerAvailable;
}

const FName& FPlasticSourceControlProvider::GetName(void) const
{
	return ProviderName;
}

FString FPlasticSourceControlProvider::GetCloudOrganization() const
{
	const int32 CloudIndex = ServerUrl.Find(TEXT("@cloud"));
	if (CloudIndex > INDEX_NONE)
	{
		return ServerUrl.Left(CloudIndex);
	}

	return FString();
}

void FPlasticSourceControlProvider::SetLastErrors(const TArray<FString>& InErrors)
{
	FScopeLock Lock(&LastErrorsCriticalSection);
	LastErrors = InErrors;
}

TArray<FString> FPlasticSourceControlProvider::GetLastErrors() const
{
	FScopeLock Lock(&LastErrorsCriticalSection);
	TArray<FString> Result = LastErrors;
	return Result;
}

ECommandResult::Type FPlasticSourceControlProvider::GetState(const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> >& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	if (!IsEnabled())
	{
		return ECommandResult::Failed;
	}

	const TArray<FString> AbsoluteFiles = SourceControlHelpers::AbsoluteFilenames(InFiles);

	if (InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		UE_LOGF(LogSourceControl, Log, "GetState: ForceUpdate");
		Execute(ISourceControlOperation::Create<FUpdateStatus>(), AbsoluteFiles);
	}

	for (const FString& AbsoluteFile : AbsoluteFiles)
	{
		OutState.Add(GetStateInternal(AbsoluteFile));
	}

	return ECommandResult::Succeeded;
}

ECommandResult::Type FPlasticSourceControlProvider::GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	if (!IsEnabled())
	{
		return ECommandResult::Failed;
	}

	if (InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		TSharedRef<class FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetChangelistsToUpdate(InChangelists);

		ISourceControlProvider::Execute(UpdatePendingChangelistsOperation, EConcurrency::Synchronous);
	}

	for (FSourceControlChangelistRef Changelist : InChangelists)
	{
		FPlasticSourceControlChangelistRef PlasticChangelist = StaticCastSharedRef<FPlasticSourceControlChangelist>(Changelist);
		OutState.Add(GetStateInternal(PlasticChangelist.Get()));
	}

	return ECommandResult::Succeeded;
}

bool FPlasticSourceControlProvider::RemoveChangelistFromCache(const FPlasticSourceControlChangelist& Changelist)
{
	return ChangelistsStateCache.Remove(Changelist) > 0;
}

TArray<FSourceControlChangelistStateRef> FPlasticSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlChangelistStateRef&)> Predicate) const
{
	TArray<FSourceControlChangelistStateRef> Result;
	for (const auto& CacheItem : ChangelistsStateCache)
	{
		FSourceControlChangelistStateRef State = CacheItem.Value;
		if (Predicate(State))
		{
			Result.Add(State);
		}
	}
	return Result;
}

TArray<FSourceControlStateRef> FPlasticSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const
{
	TArray<FSourceControlStateRef> Result;
	for (const auto& CacheItem : StateCache)
	{
		FSourceControlStateRef State = CacheItem.Value;
		if (Predicate(State))
		{
			Result.Add(State);
		}
	}
	return Result;
}

bool FPlasticSourceControlProvider::RemoveFileFromCache(const FString& Filename)
{
	return StateCache.Remove(Filename) > 0;
}

FDelegateHandle FPlasticSourceControlProvider::RegisterSourceControlStateChanged_Handle(const FSourceControlStateChanged::FDelegate& SourceControlStateChanged)
{
	return OnSourceControlStateChanged.Add(SourceControlStateChanged);
}

void FPlasticSourceControlProvider::UnregisterSourceControlStateChanged_Handle(FDelegateHandle Handle)
{
	OnSourceControlStateChanged.Remove(Handle);
}

	ECommandResult::Type FPlasticSourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	if (!bWorkspaceFound && !(InOperation->GetName() == "Connect") && !(InOperation->GetName() == "MakeWorkspace"))
	{
		UE_LOGF(LogSourceControl, Warning, "'%ls': only Connect operation allowed without a workspace", *InOperation->GetName().ToString());
		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	// Query to see if we allow this operation
	TSharedPtr<IPlasticSourceControlWorker, ESPMode::ThreadSafe> Worker = CreateWorker(InOperation->GetName());
	if (!Worker.IsValid())
	{
		// this operation is unsupported by this source control provider
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("OperationName"), FText::FromName(InOperation->GetName()));
		Arguments.Add(TEXT("ProviderName"), FText::FromName(GetName()));
		FText Message = FText::Format(LOCTEXT("UnsupportedOperation", "Operation '{OperationName}' not supported by revision control provider '{ProviderName}'"), Arguments);

		FMessageLog("SourceControl").Error(Message);
		InOperation->AddErrorMessge(Message);

		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	FPlasticSourceControlCommand* Command = new FPlasticSourceControlCommand(InOperation, Worker.ToSharedRef());
	Command->Files = SourceControlHelpers::AbsoluteFilenames(InFiles);
	Command->OperationCompleteDelegate = InOperationCompleteDelegate;

	TSharedPtr<FPlasticSourceControlChangelist, ESPMode::ThreadSafe> ChangelistPtr = StaticCastSharedPtr<FPlasticSourceControlChangelist>(InChangelist);
	Command->Changelist = ChangelistPtr ? ChangelistPtr.ToSharedRef().Get() : FPlasticSourceControlChangelist();

	// fire off operation
	if (InConcurrency == EConcurrency::Synchronous)
	{
		Command->bAutoDelete = false;

		UE_LOGF(LogSourceControl, Log, "ExecuteSynchronousCommand: %ls", *InOperation->GetName().ToString());
		return ExecuteSynchronousCommand(*Command, InOperation->GetInProgressString());
	}
	else
	{
		Command->bAutoDelete = true;

		UE_LOGF(LogSourceControl, Log, "IssueAsynchronousCommand: %ls", *InOperation->GetName().ToString());
		return IssueCommand(*Command);
	}
}

bool FPlasticSourceControlProvider::CanExecuteOperation(const FSourceControlOperationRef& InOperation) const
{
	return WorkersMap.Find(InOperation->GetName()) != nullptr;
}

bool FPlasticSourceControlProvider::CanCancelOperation(const FSourceControlOperationRef& InOperation) const
{
	return false;
}

void FPlasticSourceControlProvider::CancelOperation(const FSourceControlOperationRef& InOperation)
{
}

bool FPlasticSourceControlProvider::UsesLocalReadOnlyState() const
{
	return bUsesLocalReadOnlyState;
}

bool FPlasticSourceControlProvider::UsesChangelists() const
{
	// We don't want to show ChangeList column anymore (Unity Version Control term would be ChangeSet) BUT we need this to display the changelists in the source control menu
	return true;
}

bool FPlasticSourceControlProvider::UsesUncontrolledChangelists() const
{
	return true;
}

bool FPlasticSourceControlProvider::UsesCheckout() const
{
	return GetDefault<UPlasticSourceControlProjectSettings>()->bPromptForCheckoutOnChange;
}

bool FPlasticSourceControlProvider::UsesFileRevisions() const
{
	return IsPartialWorkspace();
}

bool FPlasticSourceControlProvider::UsesSnapshots() const
{
	return false;
}

bool FPlasticSourceControlProvider::UsesSoftRevertOnDelete() const
{
	return false;
}

bool FPlasticSourceControlProvider::AllowsDiffAgainstDepot() const
{
	return true;
}

TOptional<bool> FPlasticSourceControlProvider::HasChangesToSync() const
{
	return TOptional<bool>(); // NOTE: used by code in UE5's Status Bar but currently dormant as far as I can tell
}

TOptional<bool> FPlasticSourceControlProvider::HasChangesToCheckIn() const
{
	return TOptional<bool>(); // NOTE: used by code in UE5's Status Bar but currently dormant as far as I can tell
}

TSharedPtr<IPlasticSourceControlWorker, ESPMode::ThreadSafe> FPlasticSourceControlProvider::CreateWorker(const FName& InOperationName)
{
	const FGetPlasticSourceControlWorker* Operation = WorkersMap.Find(InOperationName);
	if (Operation != nullptr)
	{
		return Operation->Execute(*this);
	}

	return nullptr;
}

void FPlasticSourceControlProvider::RegisterWorker(const FName& InName, const FGetPlasticSourceControlWorker& InDelegate)
{
	WorkersMap.Add(InName, InDelegate);
}

void FPlasticSourceControlProvider::OutputCommandMessages(const FPlasticSourceControlCommand& InCommand) const
{
	// Note: the Perforce provider added a way to call OutputCommandMessages() from FPerforceSourceControlProvider::TryToDownloadFileFromBackgroundThread()
	// see Commit 18dc0643 by paul chipchase, 05/17/2021 11:06 AM [CL 16346666 by paul chipchase in ue5-main branch]
	// but we don't have this new API yet so we don't need this thread safety
	check(IsInGameThread()); // On the game thread we can use FMessageLog

	FMessageLog SourceControlLog("SourceControl");

	for (const FString& ErrorMessage : InCommand.ErrorMessages)
	{
		SourceControlLog.Error(FText::Format(LOCTEXT("OutputCommandMessagesFormatError", "Command: {0}, Error: {1}"),
			FText::FromName(InCommand.Operation->GetName()), FText::FromString(ErrorMessage)));
	}

	for (const FString& InfoMessage : InCommand.InfoMessages)
	{
		SourceControlLog.Info(FText::Format(LOCTEXT("OutputCommandMessagesFormatInfo", "Command: {0}, Info: {1}"),
			FText::FromName(InCommand.Operation->GetName()), FText::FromString(InfoMessage)));
	}
}

void FPlasticSourceControlProvider::UpdateWorkspaceStatus(const class FPlasticSourceControlCommand& InCommand)
{
	if (InCommand.Operation->GetName() == "Connect")
	{
		// Is connection successful?
		bServerAvailable = InCommand.bCommandSuccessful;
		bWorkspaceFound = !InCommand.WorkspaceName.IsEmpty();

		WorkspaceName = InCommand.WorkspaceName;
		RepositoryName = InCommand.RepositoryName;
		ServerUrl = InCommand.ServerUrl;

		// only pop-up errors when running in full Editor, not in command line scripts
		if (!IsRunningCommandlet())
		{
			if (bPlasticAvailable)
			{
				if (PlasticScmVersion < PlasticSourceControlVersions::OldestSupported)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("PlasticScmVersion"), FText::FromString(PlasticScmVersion.String));
					Args.Add(TEXT("OldestSupportedPlasticScmVersion"), FText::FromString(PlasticSourceControlVersions::OldestSupported.String));
					const FText UnsupportedVersionWarning = FText::Format(LOCTEXT("Plastic_UnsupportedVersion", "Unity Version Control {PlasticScmVersion} is not supported anymore by this plugin.\nUnity Version Control {OldestSupportedPlasticScmVersion} or a more recent version is required.\nPlease upgrade to the latest version."), Args);
					FMessageLog("SourceControl").Warning(UnsupportedVersionWarning);
					FMessageDialog::Open(
						EAppMsgCategory::Warning,
						EAppMsgType::Ok, UnsupportedVersionWarning
						, LOCTEXT("Plastic_UnsuportedVersionTitle", "Unsupported version!")
					);
				}
			}
			else if (InCommand.ErrorMessages.Num() > 0)
			{
				FMessageDialog::Open(
					EAppMsgCategory::Error,
					EAppMsgType::Ok, FText::FromString(InCommand.ErrorMessages[0]));
			}
		}

		if (bWorkspaceFound)
		{
			// Extend the "Source Control" menu in the Editor Toolbar on each successful connection
			PlasticSourceControlMenu.Unregister(); // cleanup for any previous connection
			PlasticSourceControlMenu.Register();
		}

		SetLastErrors(InCommand.ErrorMessages);
	}
	else if (InCommand.bConnectionDropped)
	{
		// checkconnection failed on UpdateStatus
		bServerAvailable = false;

		SetLastErrors(InCommand.ErrorMessages);
	}
	else if (!bServerAvailable)
	{
		bServerAvailable = InCommand.bCommandSuccessful;
		if (bServerAvailable)
		{
			SetLastErrors(TArray<FString>());
		}
	}

	// And for all operations running UpdateStatus, get Changeset and Branch informations:
	if (InCommand.ChangesetNumber != 0)
	{
		ChangesetNumber = InCommand.ChangesetNumber;
	}
	if (!InCommand.BranchName.IsEmpty())
	{
		BranchName = InCommand.BranchName;
	}
	if (!InCommand.WorkspaceSelector.IsEmpty())
	{
		WorkspaceSelector = InCommand.WorkspaceSelector;
	}
}

void FPlasticSourceControlProvider::Tick()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSourceControlProvider::Tick);

	bool bStatesUpdated = false;
	for (int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FPlasticSourceControlCommand& Command = *CommandQueue[CommandIndex];
		if (Command.bExecuteProcessed)
		{
			// Remove command from the queue
			CommandQueue.RemoveAt(CommandIndex);

			// Update workspace status and connection state on Connect and UpdateStatus operations
			UpdateWorkspaceStatus(Command);

			// let command update the states of any files
			bStatesUpdated |= Command.Worker->UpdateStates();

			// dump any messages to output log
			OutputCommandMessages(Command);

			if (Command.Files.Num() > 1)
			{
				UE_LOGF(LogSourceControl, Log, "%ls of %d items processed in %.3lfs", *Command.Operation->GetName().ToString(), Command.Files.Num(), (FPlatformTime::Seconds() - Command.StartTimestamp));
			}
			else if (Command.Files.Num() == 1)
			{
				UE_LOGF(LogSourceControl, Log, "%ls of %ls processed in %.3lfs", *Command.Operation->GetName().ToString(), *Command.Files[0], (FPlatformTime::Seconds() - Command.StartTimestamp));
			}
			else
			{
				UE_LOGF(LogSourceControl, Log, "%ls processed in %.3lfs", *Command.Operation->GetName().ToString(), (FPlatformTime::Seconds() - Command.StartTimestamp));
			}

			// run the completion delegate callback if we have one bound
			Command.ReturnResults();

			// commands that are left in the array during a tick need to be deleted
			if (Command.bAutoDelete)
			{
				// Only delete commands that are not running 'synchronously'
				delete &Command;
			}

			// only do one command per tick loop, as we don't want concurrent modification
			// of the command queue (which can happen in the completion delegate)
			break;
		}
	}

	if (bStatesUpdated)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSourceControlProvider::Tick::BroadcastStateUpdate);
		OnSourceControlStateChanged.Broadcast();
	}
}

TArray<TSharedRef<ISourceControlLabel>> FPlasticSourceControlProvider::GetLabels(const FString& InMatchingSpec) const
{
	TArray< TSharedRef<ISourceControlLabel> > Tags;

	// NOTE list labels. Called by CrashDebugHelper() (to remote debug Engine crash)
	//					 and by SourceControlHelpers::AnnotateFile() (to add source file to report)
	// Reserved for internal use by Epic Games with Perforce only
	return Tags;
}

TArray<FSourceControlChangelistRef> FPlasticSourceControlProvider::GetChangelists(EStateCacheUsage::Type InStateCacheUsage)
{
	if (!IsEnabled())
	{
		return TArray<FSourceControlChangelistRef>();
	}

	if (InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		TSharedRef<class FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetUpdateAllChangelists(true);

		ISourceControlProvider::Execute(UpdatePendingChangelistsOperation, EConcurrency::Synchronous);
	}

	TArray<FSourceControlChangelistRef> Changelists;
	Algo::Transform(ChangelistsStateCache, Changelists, [](const auto& Pair) { return MakeShared<FPlasticSourceControlChangelist, ESPMode::ThreadSafe>(Pair.Key); });
	return Changelists;
}

#if SOURCE_CONTROL_WITH_SLATE
TSharedRef<class SWidget> FPlasticSourceControlProvider::MakeSettingsWidget() const
{
	return SNew(SPlasticSourceControlSettings);
}
#endif

ECommandResult::Type FPlasticSourceControlProvider::ExecuteSynchronousCommand(FPlasticSourceControlCommand& InCommand, const FText& Task)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPlasticSourceControlProvider::ExecuteSynchronousCommand);

	ECommandResult::Type Result = ECommandResult::Failed;

	// Display the progress dialog if a string was provided
	{
		FScopedSourceControlProgress Progress(Task);

		// Issue the command asynchronously...
		IssueCommand(InCommand);

		// ... then wait for its completion (thus making it synchronous)
		while (!InCommand.bExecuteProcessed)
		{
			// Tick the command queue and update progress.
			Tick();

			Progress.Tick();

			// Sleep for a bit so we don't busy-wait so much.
			FPlatformProcess::Sleep(0.01f);
		}

		// always do one more Tick() to make sure the command queue is cleaned up.
		Tick();

		if (InCommand.bCommandSuccessful)
		{
			Result = ECommandResult::Succeeded;
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "Command '%ls' Failed!", *InCommand.Operation->GetName().ToString());
		}
	}

	// Delete the command now (asynchronous commands are deleted in the Tick() method)
	check(!InCommand.bAutoDelete);

	// ensure commands that are not auto deleted do not end up in the command queue
	if (CommandQueue.Contains(&InCommand))
	{
		CommandQueue.Remove(&InCommand);
	}
	delete &InCommand;

	return Result;
}

ECommandResult::Type FPlasticSourceControlProvider::IssueCommand(FPlasticSourceControlCommand& InCommand)
{
	if (GThreadPool != nullptr)
	{
		// Queue this to our worker thread(s) for resolving
		GThreadPool->AddQueuedWork(&InCommand);
		CommandQueue.Add(&InCommand);
		return ECommandResult::Succeeded;
	}
	else
	{
		// NOTE: the Perforce & Subversion providers implement this, but looking at Git history of the code I think it has never been used in UE4
		// If we need to support this, we will need to know the use cases in order to test it
		FText Message(LOCTEXT("NoSCCThreads", "There are no threads available to process the revision control command."));

		FMessageLog("SourceControl").Error(Message);
		InCommand.bCommandSuccessful = false;
		InCommand.Operation->AddErrorMessge(Message);

		return InCommand.ReturnResults();
	}
}

#undef LOCTEXT_NAMESPACE

