// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#include "IPlasticSourceControlWorker.h"
#include "PlasticSourceControlConsole.h"
#include "PlasticSourceControlMenu.h"
#include "PlasticSourceControlSettings.h"
#include "SoftwareVersion.h"

#include "ISourceControlChangelistState.h"
#include "PlasticSourceControlChangelist.h"

DECLARE_DELEGATE_RetVal_OneParam(FPlasticSourceControlWorkerRef, FGetPlasticSourceControlWorker, FPlasticSourceControlProvider&)

class FPlasticSourceControlProvider : public ISourceControlProvider
{
public:
	/** Constructor & destructor */
	FPlasticSourceControlProvider();
	~FPlasticSourceControlProvider();

	/* ISourceControlProvider implementation */
	virtual void Init(bool bForceConnection = true) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual TMap<EStatus, FString> GetStatus() const override;
	virtual bool IsEnabled() const override;
	virtual bool IsAvailable() const override;
	virtual const FName& GetName(void) const override;
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) override { return false; }
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRoot) override {}
	virtual int32 GetStateBranchIndex(const FString& InBranchName) const override { return INDEX_NONE; }
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const override { return false; }
	virtual ECommandResult::Type GetState(const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override;
	virtual ECommandResult::Type GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override;
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const override;
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle(const FSourceControlStateChanged::FDelegate& SourceControlStateChanged) override;
	virtual void UnregisterSourceControlStateChanged_Handle(FDelegateHandle Handle) override;
	virtual ECommandResult::Type Execute(const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() ) override;
	virtual bool CanExecuteOperation(const FSourceControlOperationRef& InOperation) const override;
	virtual bool CanCancelOperation(const FSourceControlOperationRef& InOperation) const override;
	virtual void CancelOperation(const FSourceControlOperationRef& InOperation) override;
	virtual bool UsesLocalReadOnlyState() const override;
	virtual bool UsesChangelists() const override;
	virtual bool UsesUncontrolledChangelists() const override;
	virtual bool UsesCheckout() const override;
	virtual bool UsesFileRevisions() const override;
	virtual bool UsesSnapshots() const override;
	virtual bool UsesSoftRevertOnDelete() const override;
	virtual bool AllowsDiffAgainstDepot() const override;
	virtual TOptional<bool> HasChangesToSync() const override;
	virtual TOptional<bool> HasChangesToCheckIn() const override;
	virtual void Tick() override;
	virtual TArray<TSharedRef<class ISourceControlLabel>> GetLabels(const FString& InMatchingSpec) const override;
	virtual TArray<FSourceControlChangelistRef> GetChangelists(EStateCacheUsage::Type InStateCacheUsage) override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const override;
#endif

	using ISourceControlProvider::Execute;

	/**
	 * Run a Plastic "version" command to check the availability of the binary and of the workspace.
	 */
	void CheckPlasticAvailability();

	/** Is Plastic command line working. */
	inline bool IsPlasticAvailable() const
	{
		return bPlasticAvailable;
	}

	/** Is Plastic workspace found. */
	inline bool IsWorkspaceFound() const
	{
		return bWorkspaceFound;
	}

	/** Get the path to the root of the Plastic workspace: can be the GameDir itself, or any parent directory */
	inline const FString& GetPathToWorkspaceRoot() const
	{
		return PathToWorkspaceRoot;
	}

	/** Get the Plastic current user */
	inline const FString& GetUserName() const
	{
		return UserName;
	}

	/** Get the Plastic current workspace */
	inline const FString& GetWorkspaceName() const
	{
		return WorkspaceName;
	}

	/** Get the Plastic current repository */
	inline const FString& GetRepositoryName() const
	{
		return RepositoryName;
	}

	/** Get the Plastic current server URL. See also GetCloudOrganization() */
	inline const FString& GetServerUrl() const
	{
		return ServerUrl;
	}

	/** Get the Name of the current branch */
	inline const FString& GetBranchName() const
	{
		return BranchName;
	}
	/** Get the Name of the current selector */
	inline const FString& GetWorkspaceSelector() const
	{
		return WorkspaceSelector;
	}
	inline void SetWorkspaceSelector(const FString& InWorkspaceSelector, const FString& InBranchName)
	{
		WorkspaceSelector = InWorkspaceSelector;
		BranchName = InBranchName;
	}

	/** Get the current Changeset Number */
	inline int32 GetChangesetNumber() const
	{
		return ChangesetNumber;
	}

	/** A partial/Gluon workspace doesn't match with a single changeset, which is identified by -1 */
	inline bool IsPartialWorkspace() const
	{
		return (ChangesetNumber == -1);
	}

	/** Version of the Unity Version Control executable used */
	inline const FSoftwareVersion& GetPlasticScmVersion() const
	{
		return PlasticScmVersion;
	}

	/** Version of the Unity Version Control plugin */
	const FString& GetPluginVersion() const
	{
		return PluginVersion;
	}

	/** Return the name of the cloud organization from the ServerUrl if applicable (MyOrganization@cloud) or an empty string */
	FString GetCloudOrganization() const;

	/** Set list of error messages that occurred after last Plastic command */
	void SetLastErrors(const TArray<FString>& InErrors);

	/** Get list of error messages that occurred after last Plastic command */
	TArray<FString> GetLastErrors() const;

	/** Helper function used to update state cache */
	TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> GetStateInternal(const FString& InFilename);

	/** Helper function used to update changelists state cache */
	TSharedRef<class FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> GetStateInternal(const FPlasticSourceControlChangelist& InChangelist);

	/**
	 * Register a worker with the provider.
	 * This is used internally so the provider can maintain a map of all available operations.
	 */
	void RegisterWorker(const FName& InName, const FGetPlasticSourceControlWorker& InDelegate);

	/** Remove a named file from the state cache */
	bool RemoveFileFromCache(const FString& Filename);

	/** Remove a changelist from the state cache */
	bool RemoveChangelistFromCache(const FPlasticSourceControlChangelist& Changelist);

	/** Returns a list of changelists from the cache based on a given predicate */
	TArray<FSourceControlChangelistStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlChangelistStateRef&)> Predicate) const;

	/** Access the Plastic source control settings */
	FPlasticSourceControlSettings& AccessSettings()
	{
		return PlasticSourceControlSettings;
	}
	const FPlasticSourceControlSettings& AccessSettings() const
	{
		return PlasticSourceControlSettings;
	}

	/** Save the Plastic source control settings */
	void SaveSettings()
	{
		PlasticSourceControlSettings.SaveSettings();
	}

private:
	/** Is Plastic binary found and working. */
	bool bPlasticAvailable = false;

	/** Is Plastic workspace found. */
	bool bWorkspaceFound = false;

	/** Indicates if source control integration is available or not. */
	bool bServerAvailable = false;

	/** Whether Unity Version Control is configured to uses local read-only state to signal whether a file is editable ("SetFilesAsReadOnly" in client.conf) */
	bool bUsesLocalReadOnlyState = false;

	/** Critical section for thread safety of error messages that occurred after last Plastic command */
	mutable FCriticalSection LastErrorsCriticalSection;

	/** List of error messages that occurred after last Plastic command */
	TArray<FString> LastErrors;

	/** Helper function for Execute() */
	TSharedPtr<class IPlasticSourceControlWorker, ESPMode::ThreadSafe> CreateWorker(const FName& InOperationName);

	/** Helper function for running command synchronously. */
	ECommandResult::Type ExecuteSynchronousCommand(class FPlasticSourceControlCommand& InCommand, const FText& Task);
	/** Issue a command asynchronously if possible. */
	ECommandResult::Type IssueCommand(class FPlasticSourceControlCommand& InCommand);

	/** Output any messages this command holds */
	void OutputCommandMessages(const class FPlasticSourceControlCommand& InCommand) const;

	/** Update workspace status on Connect and UpdateStatus operations */
	void UpdateWorkspaceStatus(const class FPlasticSourceControlCommand& InCommand);

	/** Called after a package has been saved to disk, to update the source control cache */
	void HandlePackageSaved(const FString& InPackageFilename, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);

	/** Version of the Unity Version Control executable used */
	FSoftwareVersion PlasticScmVersion;

	/** Version of the Unity Version Control plugin */
	FString PluginVersion;

	/** Path to the root of the Plastic workspace: can be the GameDir itself, or any parent directory (found by the "Connect" operation) */
	FString PathToWorkspaceRoot;

	/** Plastic current user */
	FString UserName;

	/** Plastic current workspace */
	FString WorkspaceName;

	/** Plastic current repository */
	FString RepositoryName;

	/** Plastic current server URL */
	FString ServerUrl;

	/** Name of the current branch */
	FString BranchName;

	/** Name of the object that the workspace is switched to: usually the branch, sometimes a changeset or a label */
	FString WorkspaceSelector;

	/** Current Changeset Number */
	int32 ChangesetNumber = 0;

	/** State caches */
	TMap<FString, TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> > StateCache;
	TMap<FPlasticSourceControlChangelist, TSharedRef<class FPlasticSourceControlChangelistState, ESPMode::ThreadSafe> > ChangelistsStateCache;

	/** The currently registered source control operations */
	TMap<FName, FGetPlasticSourceControlWorker> WorkersMap;

	/** Queue for commands given by the main thread */
	TArray < FPlasticSourceControlCommand* > CommandQueue;

	/** For notifying when the source control states in the cache have changed */
	FSourceControlStateChanged OnSourceControlStateChanged;

	/** Source Control Console commands */
	FPlasticSourceControlConsole PlasticSourceControlConsole;

	/** Source Control Menu Extension */
	FPlasticSourceControlMenu PlasticSourceControlMenu;

	/** The settings for Plastic source control */
	FPlasticSourceControlSettings PlasticSourceControlSettings;
};
