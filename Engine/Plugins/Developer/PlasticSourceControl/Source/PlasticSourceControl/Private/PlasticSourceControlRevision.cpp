// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlRevision.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlState.h"
#include "PlasticSourceControlUtils.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "ISourceControlModule.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

bool FPlasticSourceControlRevision::Get(FString& InOutFilename, EConcurrency::Type InConcurrency /* = EConcurrency::Synchronous */) const
{
	// if a filename for the temp file wasn't supplied generate a unique-ish one
	if (InOutFilename.IsEmpty())
	{
		// create the diff dir if we don't already have it
		IFileManager::Get().MakeDirectory(*FPaths::DiffDir(), true);
		// create a unique temp file name based on the unique revision Id
		FString TempFileName;
		if (ShelveId != ISourceControlState::INVALID_REVISION)
		{
			TempFileName = FString::Printf(TEXT("%stemp-sh%d-%s"), *FPaths::DiffDir(), ShelveId, *FPaths::GetCleanFilename(Filename));
		}
		else if (RevisionId != ISourceControlState::INVALID_REVISION)
		{
			TempFileName = FString::Printf(TEXT("%stemp-rev%d-%s"), *FPaths::DiffDir(), RevisionId, *FPaths::GetCleanFilename(Filename));
		}
		else
		{
			TempFileName = FString::Printf(TEXT("%stemp-cs%d-%s"), *FPaths::DiffDir(), ChangesetNumber, *FPaths::GetCleanFilename(Filename));
		}
		InOutFilename = FPaths::ConvertRelativePathToFull(TempFileName);
	}

	bool bCommandSuccessful = false;
	if (FPaths::FileExists(InOutFilename))
	{
		bCommandSuccessful = true; // if the temp file already exists, reuse it directly
	}
	else
	{
		FString RevisionSpecification;
		if (ShelveId != ISourceControlState::INVALID_REVISION)
		{
			// Format the revision specification of the shelved file, like rev:Content/BP.uasset#sh:33
			// Note: the plugin doesn't support shelves on Xlinks (no known RepSpec)
			RevisionSpecification = FString::Printf(TEXT("rev:%s#sh:%d"), *Filename, ShelveId);
		}
		else if (RevisionId != ISourceControlState::INVALID_REVISION)
		{
			// Format the revision specification of the file, like rev:revid:920
			RevisionSpecification = FString::Printf(TEXT("rev:revid:%d"), RevisionId);
		}
		else if (State)
		{
			// Format the revision specification of the checked-in file, like rev:Content/BP.uasset#cs:12@repo@server:8087
			RevisionSpecification = FString::Printf(TEXT("rev:%s#cs:%d@%s"), *Filename, ChangesetNumber, *State->RepSpec);
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "Unknown revision for %ls!", *Filename);
		}

		if (!RevisionSpecification.IsEmpty())
		{
			bCommandSuccessful = PlasticSourceControlUtils::RunGetFile(RevisionSpecification, InOutFilename);
		}
		if (!bCommandSuccessful && FPaths::FileExists(InOutFilename))
		{
			// On error, delete the temp file if it was created
			IFileManager::Get().Delete(*InOutFilename);
		}
	}
	return bCommandSuccessful;
}

bool FPlasticSourceControlRevision::GetAnnotated(TArray<FAnnotationLine>& OutLines) const
{
	// NOTE GetAnnotated: called only by SourceControlHelpers::AnnotateFile(),
	//      called only by ICrashDebugHelper::AddAnnotatedSourceToReport() using a changelist/check identifier
	//      called only by FCrashDebugHelperWindows::CreateMinidumpDiagnosticReport() (and Mac) to Extract annotated lines from a source file stored in Perforce, and add to the crash report.
	//      called by - MinidumpDiagnosticsApp RunMinidumpDiagnostics() for Perforce ONLY "MinidumpDiagnostics.exe <Crash.dmp> [-Annotate] [-SyncSymbols] [-SyncMicrosoftSymbols]"
	//                - FWindowsErrorReport::DiagnoseReport() (and Mac)
	// Reserved for internal use by Epic Games with Perforce only
	return false;
}

bool FPlasticSourceControlRevision::GetAnnotated(FString& InOutFilename) const
{
	// NOTE: Unused, only the above method is called by the Editor
	return false;
}

const FString& FPlasticSourceControlRevision::GetFilename() const
{
	return Filename;
}

int32 FPlasticSourceControlRevision::GetRevisionNumber() const
{
	return ChangesetNumber; // Using the Changelist as the Revision number to display in the Asset Diff Menu
}

const FString& FPlasticSourceControlRevision::GetRevision() const
{
	return Revision;
}

const FString& FPlasticSourceControlRevision::GetDescription() const
{
	return Description;
}

const FString& FPlasticSourceControlRevision::GetUserName() const
{
	return UserName;
}

const FString& FPlasticSourceControlRevision::GetClientSpec() const
{
	// Note: show Branch instead of the Workspace of the submitter since it's Perforce only
	return Branch;
}

const FString& FPlasticSourceControlRevision::GetAction() const
{
	return Action;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlRevision::GetBranchSource() const
{
	// if this revision was copied/moved from some other revision
	return BranchSource;
}

const FDateTime& FPlasticSourceControlRevision::GetDate() const
{
	return Date;
}

int32 FPlasticSourceControlRevision::GetCheckInIdentifier() const
{
	return ChangesetNumber;
}

int32 FPlasticSourceControlRevision::GetFileSize() const
{
	return FileSize;
}

#undef LOCTEXT_NAMESPACE
