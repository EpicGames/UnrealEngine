// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"

class FPlasticSourceControlState;

/** Revision of a file, linked to a specific commit */
class FPlasticSourceControlRevision : public ISourceControlRevision
{
public:
	FPlasticSourceControlRevision()
		: ChangesetNumber(ISourceControlState::INVALID_REVISION)
		, RevisionId(ISourceControlState::INVALID_REVISION)
		, Date(0)
		, FileSize(0)
	{
	}

	/** ISourceControlRevision interface */
	virtual bool Get(FString& InOutFilename, EConcurrency::Type InConcurrency = EConcurrency::Synchronous) const override;
	virtual bool GetAnnotated(TArray<FAnnotationLine>& OutLines) const override;
	virtual bool GetAnnotated(FString& InOutFilename) const override;
	virtual const FString& GetFilename() const override;
	virtual int32 GetRevisionNumber() const override;
	virtual const FString& GetRevision() const override;
	virtual const FString& GetDescription() const override;
	virtual const FString& GetUserName() const override;
	virtual const FString& GetClientSpec() const override;
	virtual const FString& GetAction() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetBranchSource() const override;
	virtual const FDateTime& GetDate() const override;
	virtual int32 GetCheckInIdentifier() const override;
	virtual int32 GetFileSize() const override;

public:
	/** Point back to State this Revision is from */
	FPlasticSourceControlState* State = nullptr;

	/** The filename this revision refers to */
	FString Filename;

	/** The changeset number of this revision */
	int32 ChangesetNumber;

	/** The internal revision ID of this file */
	int32 RevisionId;

	/** The revision to display to the user: use the changeset number */
	FString Revision;

	/** The Shelve ID instead of Changeset / Revision for case of shelved files */
	int32 ShelveId = ISourceControlState::INVALID_REVISION;

	/** The description of this revision */
	FString Description;

	/** The user that made the change */
	FString UserName;

	/** Branch where the change was made */
	FString Branch;

	/** The action (add, edit, branch etc.) performed at this revision */
	FString Action;

	/** Source of move ("branch" in Perforce term) if any */
	TSharedPtr<FPlasticSourceControlRevision, ESPMode::ThreadSafe> BranchSource;

	/** The date this revision was made */
	FDateTime Date;

	/** The size of the file at this revision */
	int32 FileSize;
};

/** History composed of the last 100 revisions of the file */
typedef TArray<TSharedRef<FPlasticSourceControlRevision, ESPMode::ThreadSafe>>	TPlasticSourceControlHistory;

typedef TSharedPtr<FPlasticSourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlRevisionPtr;
typedef TSharedRef<FPlasticSourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlRevisionRef;
