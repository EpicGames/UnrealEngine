// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlSettings.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlUtils.h"
#include "SourceControlHelpers.h"

#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformTime.h"

namespace PlasticSettingsConstants
{

/** The section of the ini file we load our settings from */
static const FString SettingsSection = TEXT("PlasticSourceControl.PlasticSourceControlSettings");

}

FString FPlasticSourceControlSettings::GetBinaryPath() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return BinaryPath;
}

bool FPlasticSourceControlSettings::SetBinaryPath(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	const bool bChanged = (BinaryPath != InString);
	if (bChanged)
	{
		BinaryPath = InString;
	}
	return bChanged;
}

bool FPlasticSourceControlSettings::GetUpdateStatusAtStartup() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return bUpdateStatusAtStartup;
}

void FPlasticSourceControlSettings::SetUpdateStatusAtStartup(const bool bInUpdateStatusAtStartup)
{
	FScopeLock ScopeLock(&CriticalSection);
	bUpdateStatusAtStartup = bInUpdateStatusAtStartup;
}

bool FPlasticSourceControlSettings::GetUpdateStatusOtherBranches() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return bUpdateStatusOtherBranches;
}

void FPlasticSourceControlSettings::SetUpdateStatusOtherBranches(const bool bInUpdateStatusOtherBranches)
{
	FScopeLock ScopeLock(&CriticalSection);
	bUpdateStatusOtherBranches = bInUpdateStatusOtherBranches;
}

bool FPlasticSourceControlSettings::GetViewLocalChanges() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return bViewLocalChanges;
}

void FPlasticSourceControlSettings::SetViewLocalChanges(const bool bInViewLocalChanges)
{
	FScopeLock ScopeLock(&CriticalSection);
	bViewLocalChanges = bInViewLocalChanges;
}

bool FPlasticSourceControlSettings::GetEnableVerboseLogs() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return bEnableVerboseLogs;
}

void FPlasticSourceControlSettings::SetEnableVerboseLogs(const bool bInEnableVerboseLogs)
{
	FScopeLock ScopeLock(&CriticalSection);
	bEnableVerboseLogs = bInEnableVerboseLogs;
}

// This is called at startup nearly before anything else in our module: BinaryPath will then be used by the provider
void FPlasticSourceControlSettings::LoadSettings()
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	GConfig->GetString(*PlasticSettingsConstants::SettingsSection, TEXT("BinaryPath"), BinaryPath, IniFile);
	GConfig->GetBool(*PlasticSettingsConstants::SettingsSection, TEXT("UpdateStatusAtStartup"), bUpdateStatusAtStartup, IniFile);
	GConfig->GetBool(*PlasticSettingsConstants::SettingsSection, TEXT("UpdateStatusOtherBranches"), bUpdateStatusOtherBranches, IniFile);
	GConfig->GetBool(*PlasticSettingsConstants::SettingsSection, TEXT("ViewLocalChanges"), bViewLocalChanges, IniFile);
	GConfig->GetBool(*PlasticSettingsConstants::SettingsSection, TEXT("EnableVerboseLogs"), bEnableVerboseLogs, IniFile);
}

void FPlasticSourceControlSettings::SaveSettings() const
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	GConfig->SetString(*PlasticSettingsConstants::SettingsSection, TEXT("BinaryPath"), *BinaryPath, IniFile);
	GConfig->SetBool(*PlasticSettingsConstants::SettingsSection, TEXT("UpdateStatusAtStartup"), bUpdateStatusAtStartup, IniFile);
	GConfig->SetBool(*PlasticSettingsConstants::SettingsSection, TEXT("UpdateStatusOtherBranches"), bUpdateStatusOtherBranches, IniFile);
	GConfig->SetBool(*PlasticSettingsConstants::SettingsSection, TEXT("ViewLocalChanges"), bViewLocalChanges, IniFile);
	GConfig->SetBool(*PlasticSettingsConstants::SettingsSection, TEXT("EnableVerboseLogs"), bEnableVerboseLogs, IniFile);
}
