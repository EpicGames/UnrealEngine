// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FPlasticSourceControlSettings
{
public:
	/** Get/Set the Plastic Binary Path */
	FString GetBinaryPath() const;
	bool SetBinaryPath(const FString& InString);

	/** Enable an asynchronous "Update Status" at Editor Startup (default is no, can take a long time). */
	bool GetUpdateStatusAtStartup() const;
	void SetUpdateStatusAtStartup(const bool bInUpdateStatusAtStartup);

	/** Enable Update status to call "history" to detect recent changesets on other branches (can be slow). */
	bool GetUpdateStatusOtherBranches() const;
	void SetUpdateStatusOtherBranches(const bool bInUpdateStatusOtherBranches);

	/** Enable the "View Changes" (changelists) window to also display locally Changed and Private files (can be slow). */
	bool GetViewLocalChanges() const;
	void SetViewLocalChanges(const bool bInViewLocalChanges);

	/** Enable LogSourceControl Verbose logs */
	bool GetEnableVerboseLogs() const;
	void SetEnableVerboseLogs(const bool bInEnableVerboseLogs);

	/** Load settings from ini file */
	void LoadSettings();

	/** Save settings to ini file */
	void SaveSettings() const;

private:
	/** A critical section for settings access */
	mutable FCriticalSection CriticalSection;

	/** Plastic binary path */
	FString BinaryPath = TEXT("cm");

	/** Run an asynchronous "Update Status" at Editor Startup (default is no).
	 * This does not work well with very big projects where this operation could take dozens of seconds
	 * preventing the project to have any source control support during this time.
	*/
	bool bUpdateStatusAtStartup = false;

	/** Enable Update status to call "history" to detect recent changesets on other branches (can be slow). */
	bool bUpdateStatusOtherBranches = false;

	/** Enable the "View Changes" (changelists) window to also display Private and locally Changed assets (even though they cannot be moved to other changelists). */
	bool bViewLocalChanges = true;

	/** Override LogSourceControl verbosity level to Verbose, and back, if not already VeryVerbose. */
	bool bEnableVerboseLogs = false;
};
