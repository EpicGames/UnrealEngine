// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlShell.h"

#include "Notification.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlProvider.h"

#include "ISourceControlModule.h"

#include "Misc/ScopeLock.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"

#if PLATFORM_LINUX
#include <sys/ioctl.h>
#endif

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h" // SECURITY_ATTRIBUTES
#undef GetUserName
#endif


#define LOCTEXT_NAMESPACE "PlasticSourceControl"


namespace PlasticSourceControlShell
{
static const TCHAR* ShellCommandResultText = TEXT("CommandResult ");
static const TCHAR* ShellUserInteractText = TEXT("Select your system [0-1]");

// In/Out Pipes for the 'cm shell' persistent child process
static void*			ShellOutputPipeRead = nullptr;
static void*			ShellOutputPipeWrite = nullptr;
static void*			ShellInputPipeRead = nullptr;
static void*			ShellInputPipeWrite = nullptr;
static FProcHandle		ShellProcessHandle;
static FCriticalSection	ShellCriticalSection;
static size_t			ShellCommandCounter = -1;
static double			ShellCumulatedTime = 0.;

// Whether we already ran a status command to warm up the current shell process
static bool             bShellIsWarmedUp = false;

// Internal function to cleanup (called under the critical section)
static void _CleanupBackgroundCommandLineShell()
{
	FPlatformProcess::ClosePipe(ShellOutputPipeRead, ShellOutputPipeWrite);
	FPlatformProcess::ClosePipe(ShellInputPipeRead, ShellInputPipeWrite);
	ShellOutputPipeRead = ShellOutputPipeWrite = nullptr;
	ShellInputPipeRead = ShellInputPipeWrite = nullptr;
}

static bool _GetShellIsWarmedUp()
{
	return bShellIsWarmedUp;
}

static void _SetShellIsWarmedUp()
{
	bShellIsWarmedUp = true;
}

// Internal function to launch the Unity Version Control background 'cm' process in interactive shell mode (called under the critical section)
static bool _StartBackgroundPlasticShell(const FString& InPathToPlasticBinary, const FString& InWorkingDirectory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlShell::_StartBackgroundPlasticShell);

	const FString FullCommand(TEXT("shell --encoding=UTF-8"));

	const bool bLaunchDetached = false;				// the new process will NOT have its own window
	const bool bLaunchHidden = true;				// the new process will be minimized in the task bar
	const bool bLaunchReallyHidden = bLaunchHidden; // the new process will not have a window or be in the task bar

	bShellIsWarmedUp = false;

	const double StartTimestamp = FPlatformTime::Seconds();

	verify(FPlatformProcess::CreatePipe(ShellOutputPipeRead, ShellOutputPipeWrite, false));	// For reading outputs from cm shell child process
	verify(FPlatformProcess::CreatePipe(ShellInputPipeRead, ShellInputPipeWrite, true));	// For writing commands to cm shell child process

#if !PLATFORM_LINUX // PLATFORM_WINDOWS || PLATFORM_MAC
	ShellProcessHandle = FPlatformProcess::CreateProc(*InPathToPlasticBinary, *FullCommand, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, *InWorkingDirectory, ShellOutputPipeWrite, ShellInputPipeRead);
#else // PLATFORM_LINUX
	// Update working directory
	char OriginalWorkingDirectory[PATH_MAX];
	getcwd(OriginalWorkingDirectory, PATH_MAX);
	chdir(TCHAR_TO_ANSI(*InWorkingDirectory));

	ShellProcessHandle = FPlatformProcess::CreateProc(*InPathToPlasticBinary, *FullCommand, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, nullptr, ShellOutputPipeWrite, ShellInputPipeRead);

	// Restore working directory
	chdir(OriginalWorkingDirectory);
#endif

	if (!ShellProcessHandle.IsValid())
	{
		UE_LOGF(LogSourceControl, Warning, "Failed to launch 'cm shell'"); // not a bug, just no Unity Version Control cli found
		_CleanupBackgroundCommandLineShell();
	}
	else
	{
		const double ElapsedTime = (FPlatformTime::Seconds() - StartTimestamp);
		UE_LOGF(LogSourceControl, Verbose, "_StartBackgroundPlasticShell: '%ls %ls' ok (in %.3lfs, handle %lld)", *InPathToPlasticBinary, *FullCommand, ElapsedTime, uint64(ShellProcessHandle.Get()));
		ShellCommandCounter = 0;
		ShellCumulatedTime = ElapsedTime;
	}

	return ShellProcessHandle.IsValid();
}

// Internal function (called under the critical section)
// bInForceExit: set to true to immediately force close the process without trying to "exit" and wait for it
static void _ExitBackgroundCommandLineShell(const bool bInForceExit = false)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlShell::_ExitBackgroundCommandLineShell);

	if (ShellProcessHandle.IsValid())
	{
		if (FPlatformProcess::IsProcRunning(ShellProcessHandle))
		{
			if (bInForceExit)
			{
				UE_LOGF(LogSourceControl, Verbose, "_ExitBackgroundCommandLineShell: TerminateProc");
				FPlatformProcess::TerminateProc(ShellProcessHandle);
			}
			else
			{
				// Tell the 'cm shell' to exit
				UE_LOGF(LogSourceControl, Verbose, "_ExitBackgroundCommandLineShell: exit...");
				FPlatformProcess::WritePipe(ShellInputPipeWrite, TEXT("exit"));
				// And wait up to one second for its termination
				const double Timeout = 1.0;
				const double StartTimestamp = FPlatformTime::Seconds();
				while (FPlatformProcess::IsProcRunning(ShellProcessHandle))
				{
					if ((FPlatformTime::Seconds() - StartTimestamp) > Timeout)
					{
						UE_LOGF(LogSourceControl, Warning, "_ExitBackgroundCommandLineShell: cm shell didn't stop gracefully in %lfs.", Timeout);
						FPlatformProcess::TerminateProc(ShellProcessHandle);
						break;
					}
					FPlatformProcess::Sleep(0.01f);
				}
			}
		}
		else
		{
			UE_LOGF(LogSourceControl, Verbose, "_ExitBackgroundCommandLineShell: 'cm shell' already stopped");
		}
		FPlatformProcess::CloseProc(ShellProcessHandle);
		_CleanupBackgroundCommandLineShell();
	}
}

// Internal function (called under the critical section)
// bInForceExit: set to true to immediately force close the process without trying to "exit" and wait for it
static void _RestartBackgroundCommandLineShell(const bool bInForceExit = false)
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const FString& PathToPlasticBinary = Provider.AccessSettings().GetBinaryPath();
	const FString& WorkingDirectory = Provider.GetPathToWorkspaceRoot();

	_ExitBackgroundCommandLineShell(bInForceExit);
	_StartBackgroundPlasticShell(PathToPlasticBinary, WorkingDirectory);
}

// Internal function (called under the critical section)
static bool _RunCommandInternal(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlShell::_RunCommandInternal);

	bool bResult = false;

	ShellCommandCounter++;

	// Detect previous crash of cm.exe and restart 'cm shell'
	if (!FPlatformProcess::IsProcRunning(ShellProcessHandle))
	{
		UE_LOGF(LogSourceControl, Warning, "RunCommand: 'cm shell' has stopped. Restarting!");
		_RestartBackgroundCommandLineShell();
	}

	// Start with the command itself ("status", "log", "checkin"...)
	FString FullCommand = InCommand;
	// Append to the command all parameters, and then finally the files
	for (const FString& Parameter : InParameters)
	{
		FullCommand += TEXT(" ");
		FullCommand += Parameter;
	}
	for (const FString& File : InFiles)
	{
		FullCommand += TEXT(" \"");
		FullCommand += File;
		FullCommand += TEXT("\"");
	}
	const FString LoggableCommand = FullCommand.Left(256); // Limit command log size to 256 characters
	UE_LOGF(LogSourceControl, Verbose, "RunCommand: '%ls' (%d chars, %d files)", *LoggableCommand, FullCommand.Len()+1, InFiles.Num());
	FullCommand += TEXT('\n'); // Finalize the command line

	// Send command to 'cm shell' process in UTF-8
	// NOTE: this explicit conversion to UTF-8 shouldn't be needed since FPlatformProcess::WritePipe() says it does it, but reading the implementation for Windows Platform show it merily truncates 16bits to 8bits chars!
	// NOTE: on the other hand, ReadPipe() does the conversion from UTF-8 correctly already!
	const FTCHARToUTF8 FullCommandUtf8(*FullCommand);
	const bool bWriteOk = FPlatformProcess::WritePipe(ShellInputPipeWrite, reinterpret_cast<const uint8*>(FullCommandUtf8.Get()), FullCommandUtf8.Length());

	// And wait up to 180.0 seconds for any kind of output from cm shell: in case of lengthier operation, intermediate output (like percentage of progress) is expected, which would refresh the timeout
	static const double Timeout = 180.0;
	const double StartTimestamp = FPlatformTime::Seconds();
	double LastActivity = StartTimestamp;
	double LastLog = StartTimestamp;
	static const double LogInterval = 5.0;
	int32 PreviousLogLen = 0;
	while (FPlatformProcess::IsProcRunning(ShellProcessHandle))
	{
		FString Output = FPlatformProcess::ReadPipe(ShellOutputPipeRead);
		if (!Output.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlShell::_RunCommandInternal::ParseOutput);

			LastActivity = FPlatformTime::Seconds(); // freshen the timestamp while cm is still actively outputting information
			OutResults.Append(MoveTemp(Output));
			// Search the output for the line containing the result code, also indicating the end of the command
			const uint32 IndexCommandResult = OutResults.Find(ShellCommandResultText, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (INDEX_NONE != IndexCommandResult)
			{
				const uint32 IndexEndResult = OutResults.Find(pchDelim, ESearchCase::CaseSensitive, ESearchDir::FromStart, IndexCommandResult + 14);
				if (INDEX_NONE != IndexEndResult)
				{
					const FString Result = OutResults.Mid(IndexCommandResult + 14, IndexEndResult - IndexCommandResult - 14);
					const int32 ResultCode = FCString::Atoi(*Result);
					bResult = (ResultCode == 0);
					// remove the CommandResult line from the OutResults
					OutResults.RemoveAt(IndexCommandResult, OutResults.Len() - IndexCommandResult);
					break;
				}
			}

			// Search the output for a potential user interaction request (in case the authentication token isn't saved or valid anymore)
			const uint32 IndexPrompt = OutResults.Find(ShellUserInteractText, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (INDEX_NONE != IndexPrompt)
			{
				const FText ShellRequiresInteractionError(LOCTEXT("SourceControlShell_AskAuthenticate", "Unity Version Control command line requires user interaction.\nSign in using the Unity Version Control client."));
				FNotification::DisplayFailure(ShellRequiresInteractionError);

				// Restart the shell without waiting, it is forever blocked waiting for user input
				_RestartBackgroundCommandLineShell(true);
				break; // and quit the loop
			}
		}
		else if ((FPlatformTime::Seconds() - LastLog > LogInterval) && (PreviousLogLen < OutResults.Len()))
		{
			// In case of long running operation, start to print intermediate output from cm shell (like percentage of progress)
			UE_LOGF(LogSourceControl, Log, "RunCommand: '%ls' in progress for %.3lfs... (%d chars):\n%ls", *InCommand, (FPlatformTime::Seconds() - StartTimestamp), OutResults.Len() - PreviousLogLen, *OutResults.Mid(PreviousLogLen));
			PreviousLogLen = OutResults.Len();
			LastLog = FPlatformTime::Seconds(); // freshen the timestamp of last log
		}
		else if (FPlatformTime::Seconds() - LastActivity > Timeout)
		{
			// In case of timeout, ask the blocking 'cm shell' process to exit, detach from it and restart it immediately
			UE_LOGF(LogSourceControl, Error, "RunCommand: '%ls' TIMEOUT after %.3lfs output (%d chars):\n%ls", *InCommand, (FPlatformTime::Seconds() - StartTimestamp), OutResults.Len(), *OutResults.Mid(PreviousLogLen));
			_RestartBackgroundCommandLineShell(true);
			// Return output results as error so they get propagated to the Message Log window
			OutErrors = MoveTemp(OutResults);
			return false;
		}
		else if (IsEngineExitRequested())
		{
			UE_LOGF(LogSourceControl, Warning, "RunCommand: '%ls' Engine Exit was requested after %.3lfs output (%d chars):\n%ls", *InCommand, (FPlatformTime::Seconds() - StartTimestamp), OutResults.Len() - PreviousLogLen, *OutResults.Mid(PreviousLogLen));
			_ExitBackgroundCommandLineShell();
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlShell::_RunCommandInternal::Sleep);
		FPlatformProcess::Sleep(0.001f);
	}
	const double ElapsedTime = (FPlatformTime::Seconds() - StartTimestamp);

	if (!InCommand.Equals(TEXT("exit")))
	{
		if (!FPlatformProcess::IsProcRunning(ShellProcessHandle))
		{
			// 'cm shell' normally only terminates in case of 'exit' command. Will restart on next command.
			UE_LOGF(LogSourceControl, Error, "RunCommand: '%ls' 'cm shell' stopped after %.3lfs output (%d chars):\n%ls", *LoggableCommand, ElapsedTime, OutResults.Len(), *OutResults.Left(4096)); // Limit result size to 4096 characters
		}
		else if (!bResult)
		{
			UE_LOGF(LogSourceControl, Warning, "RunCommand: '%ls' (in %.3lfs) output (%d chars):\n%ls", *LoggableCommand, ElapsedTime, OutResults.Len(), *OutResults.Left(4096)); // Limit result size to 4096 characters
		}
		else
		{
			if (PreviousLogLen > 0)
			{
				UE_LOGF(LogSourceControl, Log, "RunCommand: '%ls' (in %.3lfs) output (%d chars):\n%ls", *LoggableCommand, ElapsedTime, OutResults.Len(), *OutResults.Mid(PreviousLogLen).Left(4096)); // Limit result size to 4096 characters
			}
			else
			{
				if (OutResults.Len() <= 200) // Limit result size to 200 characters
				{
					UE_LOGF(LogSourceControl, Log, "RunCommand: '%ls' (in %.3lfs) output (%d chars):\n%ls", *LoggableCommand, ElapsedTime, OutResults.Len(), *OutResults);
				}
				else
				{
					UE_LOGF(LogSourceControl, Log, "RunCommand: '%ls' (in %.3lfs) (output %d chars not displayed)", *LoggableCommand, ElapsedTime, OutResults.Len());
					UE_LOGF(LogSourceControl, Verbose, "\n%ls", *OutResults.Left(4096));; // Limit result size to 4096 characters
				}
			}
		}
	}
	// Return output as error if result code is an error
	if (!bResult)
	{
		OutErrors = MoveTemp(OutResults);
	}

	ShellCumulatedTime += ElapsedTime;
	UE_LOGF(LogSourceControl, Verbose, "RunCommand: cumulated time spent in shell: %.3lfs (count %zu)", ShellCumulatedTime, ShellCommandCounter);

	return bResult;
}

// Launch the Unity Version Control 'cm shell' process in background for optimized successive commands (thread-safe)
bool Launch(const FString& InPathToPlasticBinary, const FString& InWorkingDirectory)
{
	// Protect public APIs from multi-thread access
	FScopeLock Lock(&ShellCriticalSection);

	// terminate previous shell if one is already running
	_ExitBackgroundCommandLineShell();

	return _StartBackgroundPlasticShell(InPathToPlasticBinary, InWorkingDirectory);
}

// Terminate the background 'cm shell' process and associated pipes (thread-safe)
void Terminate()
{
	// Protect public APIs from multi-thread access
	FScopeLock Lock(&ShellCriticalSection);

	_ExitBackgroundCommandLineShell();
}

void SetShellIsWarmedUp()
{
	FScopeLock Lock(&ShellCriticalSection);

	_SetShellIsWarmedUp();
}

bool GetShellIsWarmedUp()
{
	FScopeLock Lock(&ShellCriticalSection);

	return _GetShellIsWarmedUp();
}

// Run command and return the raw result
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors)
{
	// Protect public APIs from multi-thread access
	FScopeLock Lock(&ShellCriticalSection);

	return _RunCommandInternal(InCommand, InParameters, InFiles, OutResults, OutErrors);
}

} // namespace PlasticSourceControlShell

#undef LOCTEXT_NAMESPACE
