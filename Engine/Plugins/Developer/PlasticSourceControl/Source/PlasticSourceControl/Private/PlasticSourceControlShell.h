// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace PlasticSourceControlShell
{
#if PLATFORM_WINDOWS
	constexpr const TCHAR* pchDelim = TEXT("\r\n");
#else
	constexpr const TCHAR* pchDelim = TEXT("\n");
#endif


/**
 * Launch the Unity Version Control "shell" command line process to run it in the background.
 *
 * @param	InPathToPlasticBinary	The path to the Plastic binary
 * @param	InWorkspaceRoot			The workspace from where to run the command - usually the Game directory
 * @returns true if the command succeeded and returned no errors
 */
bool Launch(const FString& InPathToPlasticBinary, const FString& InWorkspaceRoot);

/** Terminate the background 'cm shell' process and associated pipes */
void Terminate();

/** Mark the current shell process as already warmed up - i.e. we already ran a preliminary 'status' command. */
void SetShellIsWarmedUp();

/**
 * Retrieve whether the current shell process was already warmed up - i.e. we already ran a preliminary 'status' command.
 *
 * @returns true if the shell process was already warmed up
 */
bool GetShellIsWarmedUp();


/**
 * Run a Plastic command - the result is the output of cm, as a multi-line string.
 *
 * @param	InCommand			The Plastic command - e.g. commit
 * @param	InParameters		The parameters to the Plastic command
 * @param	InFiles				The files to be operated on
 * @param	OutResults			The results (from StdOut) as a multi-line string.
 * @param	OutErrors			Any errors (from StdErr) as a multi-line string.
 * @returns true if the command succeeded and returned no errors
 */
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors);

} // namespace PlasticSourceControlShell
