// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlState.h"
#include "PlasticSourceControlProjectSettings.h"

#include "Misc/Paths.h"
#include "ISourceControlModule.h"
#include "Styling/AppStyle.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl.State"

const TCHAR* FPlasticSourceControlState::ToString() const
{
	switch (WorkspaceState)
	{
		case EWorkspaceState::Ignored: return TEXT("Ignored");
		case EWorkspaceState::Controlled: return TEXT("Controlled");
		case EWorkspaceState::CheckedOutChanged: return TEXT("Checked-out (changed)");
		case EWorkspaceState::CheckedOutUnchanged: return TEXT("Checked-out (unchanged)");
		case EWorkspaceState::Added: return TEXT("Added");
		case EWorkspaceState::Moved: return TEXT("Moved");
		case EWorkspaceState::Copied: return TEXT("Copied");
		case EWorkspaceState::Replaced: return TEXT("Replaced");
		case EWorkspaceState::Deleted: return TEXT("Removed");
		case EWorkspaceState::LocallyDeleted: return TEXT("Deleted locally");
		case EWorkspaceState::Changed: return TEXT("Changed");
		case EWorkspaceState::Conflicted: return TEXT("Conflicted");
		case EWorkspaceState::Private: return TEXT("Private");
		case EWorkspaceState::Unknown:
		default: return TEXT("Unknown");
	}
}

FText FPlasticSourceControlState::ToText() const
{
	static const FText Ignored = LOCTEXT("Ignored", "Ignored");
	static const FText Controlled = LOCTEXT("Controlled", "Controlled");
	static const FText CheckedOutChanged = LOCTEXT("CheckedOutChanged", "Checked-out (changed)");
	static const FText CheckedOutUnchanged = LOCTEXT("CheckedOutUnchanged", "Checked-out (unchanged)");
	static const FText Added = LOCTEXT("Added", "Added");
	static const FText Moved = LOCTEXT("Moved", "Moved");
	static const FText Copied = LOCTEXT("Copied", "Copied");
	static const FText Replaced = LOCTEXT("Replaced", "Replaced");
	static const FText Deleted = LOCTEXT("Deleted", "Removed");
	static const FText LocallyDeleted = LOCTEXT("LocallyDeleted", "Deleted locally");
	static const FText Changed = LOCTEXT("Changed", "Changed");
	static const FText Conflicted = LOCTEXT("Conflicted", "Conflicted");
	static const FText Private = LOCTEXT("Private", "Private");
	static const FText Unknown = LOCTEXT("Unknown", "Unknown");
	switch (WorkspaceState)
	{
		case EWorkspaceState::Ignored: return Ignored;
		case EWorkspaceState::Controlled: return Controlled;
		case EWorkspaceState::CheckedOutChanged: return CheckedOutChanged;
		case EWorkspaceState::CheckedOutUnchanged: return CheckedOutUnchanged;
		case EWorkspaceState::Added: return Added;
		case EWorkspaceState::Moved: return Moved;
		case EWorkspaceState::Copied: return Copied;
		case EWorkspaceState::Replaced: return Replaced;
		case EWorkspaceState::Deleted: return Deleted;
		case EWorkspaceState::LocallyDeleted: return LocallyDeleted;
		case EWorkspaceState::Changed: return Changed;
		case EWorkspaceState::Conflicted: return Conflicted;
		case EWorkspaceState::Private: return Private;
		case EWorkspaceState::Unknown:
		default: return Unknown;
	}
}

int32 FPlasticSourceControlState::GetHistorySize() const
{
	return History.Num();
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlState::GetHistoryItem(int32 HistoryIndex) const
{
	check(History.IsValidIndex(HistoryIndex));
	return History[HistoryIndex];
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlState::FindHistoryRevision(int32 RevisionNumber) const
{
	for (const auto& Revision : History)
	{
		if (Revision->GetRevisionNumber() == RevisionNumber)
		{
			return Revision;
		}
	}

	return nullptr;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlState::FindHistoryRevision(const FString& InRevision) const
{
	for (const auto& Revision : History)
	{
		if (Revision->GetRevision() == InRevision)
		{
			return Revision;
		}
	}

	return nullptr;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FPlasticSourceControlState::GetCurrentRevision() const
{
	for (const auto& Revision : History)
	{
		// look for the changeset number, not the revision
		if (Revision->ChangesetNumber == LocalRevisionChangeset)
		{
			return Revision;
		}
	}

	return nullptr;
}

ISourceControlState::FResolveInfo FPlasticSourceControlState::GetResolveInfo() const
{
	return PendingResolveInfo;
}

#if SOURCE_CONTROL_WITH_SLATE

FSlateIcon FPlasticSourceControlState::GetIcon() const
{
	if (!IsCurrent())
	{
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.NotAtHeadRevision");
	}

	if (!IsCheckedOutImplementation())
	{
		if (IsCheckedOutOther())
		{
			return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOutByOtherUser", NAME_None, "RevisionControl.CheckedOutByOtherUserBadge");
		}

		if (IsRetainedInOtherBranch())
		{
			return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOutByOtherUserOtherBranch", NAME_None, "RevisionControl.CheckedOutByOtherUserOtherBranchBadge");
		}

		if (IsModifiedInOtherBranch())
		{
			return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.ModifiedOtherBranch", NAME_None, "RevisionControl.ModifiedBadge");
		}
	}

	switch (WorkspaceState)
	{
	case EWorkspaceState::CheckedOutChanged:
	case EWorkspaceState::CheckedOutUnchanged:
	case EWorkspaceState::Replaced: // Merged (waiting for check-in)
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOut");
	case EWorkspaceState::Added:
	case EWorkspaceState::Copied:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.OpenForAdd");
	case EWorkspaceState::Moved:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Branched");
	case EWorkspaceState::Deleted:
	case EWorkspaceState::LocallyDeleted:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.MarkedForDelete");
	case EWorkspaceState::Conflicted:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Conflicted");
	case EWorkspaceState::Private: // Not controlled
	case EWorkspaceState::Changed: // Changed but unchecked-out is in a certain way not controlled
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.NotInDepot");
	case EWorkspaceState::Ignored:
	case EWorkspaceState::Unknown:
	case EWorkspaceState::Controlled: // Unchanged (not checked out) ie no icon
	default:
		return FSlateIcon();
	}
}

#endif // SOURCE_CONTROL_WITH_SLATE

FText FPlasticSourceControlState::GetDisplayName() const
{
	FNumberFormattingOptions NoCommas;
	NoCommas.UseGrouping = false;
	if (!IsCurrent())
	{
		return FText::Format(LOCTEXT("NotCurrent", "Not at the head revision CS:{0} {1} (local revision is CS:{2})"),
			FText::AsNumber(DepotRevisionChangeset), FText::FromString(HeadUserName), FText::AsNumber(LocalRevisionChangeset, &NoCommas));
	}

	if (!IsCheckedOutImplementation())
	{
		if (IsCheckedOutOther())
		{
			return FText::Format(LOCTEXT("CheckedOutOther", "Checked out by {0} on {1} (in {2}) since {3}"), FText::FromString(LockedBy), FText::FromString(LockedBranch), FText::FromString(LockedWhere), FText::AsDateTime(LockedDate));
		}

		if (IsRetainedInOtherBranch())
		{
			return FText::Format(LOCTEXT("RetainedLock", "Retained on {0} by {1} since {2}"), FText::FromString(LockedBranch), FText::FromString(RetainedBy), FText::AsDateTime(LockedDate));
		}

		if (IsModifiedInOtherBranch())
		{
			return FText::Format(LOCTEXT("ModifiedOtherBranch", "Modified in {0} as CS:{1} by {2} (local revision is CS:{3})"),
				FText::FromString(HeadBranch), FText::AsNumber(HeadChangeList, &NoCommas), FText::FromString(HeadUserName), FText::AsNumber(LocalRevisionChangeset, &NoCommas));
		}
	}

	switch (WorkspaceState)
	{
	case EWorkspaceState::Unknown:
		return LOCTEXT("Unknown", "Unknown");
	case EWorkspaceState::Ignored:
		return LOCTEXT("Ignored", "Ignored");
	case EWorkspaceState::Controlled:
		return LOCTEXT("Controlled", "Controlled");
	case EWorkspaceState::CheckedOutChanged:
		return LOCTEXT("CheckedOutChanged", "Checked-out (changed)");
	case EWorkspaceState::CheckedOutUnchanged:
		return LOCTEXT("CheckedOutUnchanged", "Checked-out (unchanged)");
	case EWorkspaceState::Added:
		return LOCTEXT("Added", "Added");
	case EWorkspaceState::Moved:
		return LOCTEXT("Moved", "Moved");
	case EWorkspaceState::Copied:
		return LOCTEXT("Copied", "Copied");
	case EWorkspaceState::Replaced:
		return LOCTEXT("Replaced", "Replaced");
	case EWorkspaceState::Deleted:
		return LOCTEXT("Deleted", "Removed");
	case EWorkspaceState::LocallyDeleted:
		return LOCTEXT("LocallyDeleted", "Deleted locally");
	case EWorkspaceState::Changed:
		return LOCTEXT("Changed", "Changed");
	case EWorkspaceState::Conflicted:
		return LOCTEXT("Conflicted", "Conflicted");
	case EWorkspaceState::Private:
		return LOCTEXT("NotControlled", "Not Under Revision Control");
	}

	return FText();
}

FText FPlasticSourceControlState::GetDisplayTooltip() const
{
	FNumberFormattingOptions NoCommas;
	NoCommas.UseGrouping = false;
	if (!IsCurrent())
	{
		return FText::Format(LOCTEXT("NotCurrent_Tooltip", "Not at the head revision CS:{0} {1} (local revision is CS:{2})"),
			FText::AsNumber(DepotRevisionChangeset), FText::FromString(HeadUserName), FText::AsNumber(LocalRevisionChangeset, &NoCommas));
	}

	if (!IsCheckedOutImplementation())
	{
		if (IsCheckedOutOther())
		{
			return FText::Format(LOCTEXT("CheckedOutOther_Tooltip", "Checked out by {0} on {1} (in {2}) since {3}"), FText::FromString(LockedBy), FText::FromString(LockedBranch), FText::FromString(LockedWhere), FText::AsDateTime(LockedDate));
		}

		if (IsRetainedInOtherBranch())
		{
			return FText::Format(LOCTEXT("RetainedLock_Tooltip", "Retained on {0} by {1} since {2}"), FText::FromString(LockedBranch), FText::FromString(RetainedBy), FText::AsDateTime(LockedDate));
		}

		if (IsModifiedInOtherBranch())
		{
			return FText::Format(LOCTEXT("ModifiedOtherBranch_Tooltip", "Modified in {0} as CS:{1} by {2} (local revision is CS:{3})"),
				FText::FromString(HeadBranch), FText::AsNumber(HeadChangeList, &NoCommas), FText::FromString(HeadUserName), FText::AsNumber(LocalRevisionChangeset, &NoCommas));
		}
	}

	switch (WorkspaceState)
	{
	case EWorkspaceState::Unknown:
		return FText();
	case EWorkspaceState::Ignored:
		return LOCTEXT("Ignored_Tooltip", "Ignored");
	case EWorkspaceState::Controlled:
		return FText();
	case EWorkspaceState::CheckedOutChanged:
		return LOCTEXT("CheckedOutChanged_Tooltip", "Checked-out (changed)");
	case EWorkspaceState::CheckedOutUnchanged:
		return LOCTEXT("CheckedOutUnchanged_Tooltip", "Checked-out (unchanged)");
	case EWorkspaceState::Added:
		return LOCTEXT("Added_Tooltip", "Added");
	case EWorkspaceState::Moved:
	{
		FString MoveOrigin = MovedFrom;
		FPaths::MakePathRelativeTo(MoveOrigin, *LocalFilename);
		return FText::Format(LOCTEXT("Moved_Tooltip", "Moved from {0}"),
			FText::FromString(FPaths::GetBaseFilename(MoveOrigin, false)));
	}
	case EWorkspaceState::Copied:
		return LOCTEXT("Copied_Tooltip", "Copied");
	case EWorkspaceState::Replaced:
		return LOCTEXT("Replaced_Tooltip", "Replaced (merged)");
	case EWorkspaceState::Deleted:
		return LOCTEXT("Deleted_Tooltip", "Removed");
	case EWorkspaceState::LocallyDeleted:
		return LOCTEXT("LocallyDeleted_Tooltip", "Deleted locally");
	case EWorkspaceState::Changed:
		return LOCTEXT("Modified_Tooltip", "Changed locally");
	case EWorkspaceState::Conflicted:
		return FText::Format(LOCTEXT("Conflicted_Tooltip", "Conflict merging from source/remote CS:{0} into target/local CS:{1})"),
			FText::FromString(PendingResolveInfo.RemoteRevision), FText::AsNumber(LocalRevisionChangeset, &NoCommas));
	case EWorkspaceState::Private:
		return LOCTEXT("NotControlled_Tooltip", "Private: not under revision control");
	}

	return FText();
}

const FString& FPlasticSourceControlState::GetFilename() const
{
	return LocalFilename;
}

const FDateTime& FPlasticSourceControlState::GetTimeStamp() const
{
	return TimeStamp;
}

bool FPlasticSourceControlState::CanCheckIn() const
{
	// Deleted assets don't appear in the Content Browser but in Submit to Source Control Window
	const bool bCanCheckIn =  (WorkspaceState == EWorkspaceState::Added
							|| WorkspaceState == EWorkspaceState::Deleted
							|| WorkspaceState == EWorkspaceState::LocallyDeleted
							|| WorkspaceState == EWorkspaceState::Changed
							|| WorkspaceState == EWorkspaceState::Moved
							|| WorkspaceState == EWorkspaceState::Copied
							|| WorkspaceState == EWorkspaceState::Replaced
							|| WorkspaceState == EWorkspaceState::CheckedOutChanged)
							&& !IsCheckedOutOther()	// Is not already checked-out elsewhere
							&& IsCurrent();			// Is up to date (at the revision of the repo)

	if (!IsUnknown())
	{
		UE_LOGF(LogSourceControl, Verbose, "%ls CanCheckIn=%d", *LocalFilename, bCanCheckIn);
	}

	return bCanCheckIn;
}

bool FPlasticSourceControlState::CanCheckout() const
{
	if (!GetDefault<UPlasticSourceControlProjectSettings>()->bPromptForCheckoutOnChange)
	{
		return false;
	}

	const bool bCanCheckout  =    (WorkspaceState == EWorkspaceState::Controlled	// In source control, Unmodified
								|| WorkspaceState == EWorkspaceState::Changed)		// In source control, but not checked-out
							//	&& !IsCheckedOutOther()	// Note: with multi-destination branches it is sometimes possible to checkout an asset already checked-out elsewhere. Let the user try.
								&& IsCurrent();			// Is up to date (at the revision of the repo)

	if (!IsUnknown())
	{
		UE_LOGF(LogSourceControl, Verbose, "%ls CanCheckout=%d", *LocalFilename, bCanCheckout);
	}

	return bCanCheckout;
}

bool FPlasticSourceControlState::IsCheckedOut() const
{
	const bool bIsCheckedOut = IsCheckedOutImplementation()
							|| WorkspaceState == EWorkspaceState::Changed;		// Note: Workaround to enable checkin (still required by UE5.0)

	if (bIsCheckedOut)
	{
		UE_LOGF(LogSourceControl, Verbose, "%ls IsCheckedOut", *LocalFilename);
	}

	if (GetDefault<UPlasticSourceControlProjectSettings>()->bPromptForCheckoutOnChange)
	{
		return bIsCheckedOut;
	}
	else
	{
		// Any controlled state will be considered as checked out if the prompt is disabled
		return IsSourceControlled();
	}
}

bool FPlasticSourceControlState::IsPendingChanges() const
{
	return WorkspaceState != EWorkspaceState::Unknown
		&& WorkspaceState != EWorkspaceState::Ignored
		&& WorkspaceState != EWorkspaceState::Controlled;
}

bool FPlasticSourceControlState::IsCheckedOutImplementation() const
{
	const bool bIsCheckedOut = WorkspaceState == EWorkspaceState::CheckedOutChanged
							|| WorkspaceState == EWorkspaceState::CheckedOutUnchanged
							|| WorkspaceState == EWorkspaceState::Added
							|| WorkspaceState == EWorkspaceState::Deleted
							|| WorkspaceState == EWorkspaceState::Copied
							|| WorkspaceState == EWorkspaceState::Moved
							|| WorkspaceState == EWorkspaceState::Conflicted	// In source control, waiting for merge
							|| WorkspaceState == EWorkspaceState::Replaced;		// In source control, merged, waiting for checkin to conclude the merge

	return bIsCheckedOut;
}

bool FPlasticSourceControlState::IsLocked() const
{
	return !LockedBy.IsEmpty();
}

bool FPlasticSourceControlState::IsRetainedInOtherBranch() const
{
	return !RetainedBy.IsEmpty();
}

bool FPlasticSourceControlState::IsCheckedOutOther(FString* Who) const
{
	if (Who != NULL)
	{
		*Who = LockedBy;
	}

	// An asset is locked somewhere else if it is Locked but not CheckedOut on the current workspace
	const bool bIsLockedByOther = IsLocked() && !IsCheckedOutImplementation();

	if (bIsLockedByOther)
	{
		UE_LOGF(LogSourceControl, VeryVerbose, "%ls IsCheckedOutOther by '%ls' (%ls)", *LocalFilename, *LockedBy, *LockedWhere);
	}

	return bIsLockedByOther;
}

/** Get whether this file is checked out in a different branch */
bool FPlasticSourceControlState::IsCheckedOutInOtherBranch(const FString& CurrentBranch /* = FString() */) const
{
	// NOTE: technically this scenario isn't currently possible with Unity Version Control,
	//       but the plugin need to use an existing Engine hook, so it's using this one as a way to display "Retained" locks
	return IsRetainedInOtherBranch();
}

/** Get whether this file is modified in a different branch */
bool FPlasticSourceControlState::IsModifiedInOtherBranch(const FString& CurrentBranch /* = FString() */) const
{
	return !HeadBranch.IsEmpty();
}

/** Get head modification information for other branches
 * @returns true with parameters populated if there is a branch with a newer modification (edit/delete/etc)
*/
bool FPlasticSourceControlState::GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const
{
	HeadBranchOut = HeadBranch;
	ActionOut = HeadAction;
	HeadChangeListOut = HeadChangeList;

	return !HeadBranch.IsEmpty();
}

bool FPlasticSourceControlState::IsCurrent() const
{
	// NOTE: Deleted assets get a "-1" HeadRevision which we do not want to override the real icon state
	const bool bIsCurrent = (LocalRevisionChangeset == DepotRevisionChangeset) || (WorkspaceState == EWorkspaceState::Deleted);

	if (bIsCurrent)
	{
		UE_LOGF(LogSourceControl, VeryVerbose, "%ls IsCurrent", *LocalFilename);
	}

	return bIsCurrent;
}

bool FPlasticSourceControlState::IsSourceControlled() const
{
	// NOTE: the Editor Collections rely on the default 'Unknown' state (until the actual file status is obtained) to be considered "in source control"
	const bool bIsSourceControlled = WorkspaceState != EWorkspaceState::Private
								  && WorkspaceState != EWorkspaceState::Ignored;
								  // WorkspaceState != EWorkspaceState::Unknown

	if (!bIsSourceControlled && !IsUnknown()) UE_LOGF(LogSourceControl, Verbose, "%ls NOT SourceControlled", *LocalFilename);

	return bIsSourceControlled;
}

bool FPlasticSourceControlState::IsAdded() const
{
	const bool bIsAdded =	WorkspaceState == EWorkspaceState::Added
						 || WorkspaceState == EWorkspaceState::Copied;

	if (bIsAdded) UE_LOGF(LogSourceControl, Verbose, "%ls IsAdded", *LocalFilename);

	return bIsAdded;
}

bool FPlasticSourceControlState::IsDeleted() const
{
	const bool bIsDeleted =	WorkspaceState == EWorkspaceState::Deleted
						 || WorkspaceState == EWorkspaceState::LocallyDeleted;

	if (bIsDeleted) UE_LOGF(LogSourceControl, Verbose, "%ls IsDeleted", *LocalFilename);

	return bIsDeleted;
}

bool FPlasticSourceControlState::IsIgnored() const
{
	if (WorkspaceState == EWorkspaceState::Ignored) UE_LOGF(LogSourceControl, Verbose, "%ls IsIgnored", *LocalFilename);

	return WorkspaceState == EWorkspaceState::Ignored;
}

bool FPlasticSourceControlState::CanEdit() const
{
	const bool bCanEdit =  WorkspaceState == EWorkspaceState::CheckedOutChanged
						|| WorkspaceState == EWorkspaceState::CheckedOutUnchanged
						|| WorkspaceState == EWorkspaceState::Added
						|| WorkspaceState == EWorkspaceState::Moved
						|| WorkspaceState == EWorkspaceState::Copied
						|| WorkspaceState == EWorkspaceState::Replaced;

	UE_LOGF(LogSourceControl, Verbose, "%ls CanEdit=%d", *LocalFilename, bCanEdit);

	return bCanEdit;
}

bool FPlasticSourceControlState::CanDelete() const
{
	return !IsCheckedOutOther() && IsSourceControlled() && IsCurrent();
}

bool FPlasticSourceControlState::IsUnknown() const
{
	return WorkspaceState == EWorkspaceState::Unknown;
}

bool FPlasticSourceControlState::IsModified() const
{
	// Warning: for a clean "checkin" (commit) checked-out files unmodified should be removed from the changeset (Perforce)
	//
	// Thus, before checkin UE4 Editor call RevertUnchangedFiles() in PromptForCheckin() and CheckinFiles().
	//
	// So here we must take care to enumerate all states that need to be committed, all other will be discarded:
	//  - Unknown
	//  - Controlled (Unchanged)
	//  - CheckedOutUnchanged
	//  - Private (Not Controlled)
	//  - Ignored
	const bool bIsModified =   WorkspaceState == EWorkspaceState::CheckedOutChanged
							|| WorkspaceState == EWorkspaceState::Added
							|| WorkspaceState == EWorkspaceState::Moved
							|| WorkspaceState == EWorkspaceState::Copied
							|| WorkspaceState == EWorkspaceState::Replaced
							|| WorkspaceState == EWorkspaceState::Deleted
							|| WorkspaceState == EWorkspaceState::LocallyDeleted
							|| WorkspaceState == EWorkspaceState::Changed
							|| WorkspaceState == EWorkspaceState::Conflicted;

	UE_LOGF(LogSourceControl, Verbose, "%ls IsModified=%d", *LocalFilename, bIsModified);

	return bIsModified;
}


bool FPlasticSourceControlState::CanAdd() const
{
	if (!IsUnknown()) UE_LOGF(LogSourceControl, Verbose, "%ls CanAdd=%d", *LocalFilename, WorkspaceState == EWorkspaceState::Private);

	return WorkspaceState == EWorkspaceState::Private;
}

bool FPlasticSourceControlState::IsConflicted() const
{
	if (WorkspaceState == EWorkspaceState::Conflicted) UE_LOGF(LogSourceControl, Verbose, "%ls IsConflicted", *LocalFilename);

	return WorkspaceState == EWorkspaceState::Conflicted;
}

bool FPlasticSourceControlState::CanRevert() const
{
	return IsModified() || WorkspaceState == EWorkspaceState::CheckedOutUnchanged;
}

#undef LOCTEXT_NAMESPACE