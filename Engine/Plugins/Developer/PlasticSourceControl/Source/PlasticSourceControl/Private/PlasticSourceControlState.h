// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlState.h"
#include "ISourceControlRevision.h"
#include "PlasticSourceControlRevision.h"

#include "PlasticSourceControlChangelist.h"

enum class EWorkspaceState
{
	Unknown,
	Ignored,
	Controlled, // called "Pristine" in Perforce, "Unchanged" in Git, "Clean" in SVN
	CheckedOutChanged, // Checked-out, with changes (or without knowing for older version of Unity Version Control)
	CheckedOutUnchanged, // Checked-out with no changes (cannot be checked-in and can be reverted by UndoUnchanged)
	Added,
	Moved, // Renamed
	Copied,
	Replaced, // Replaced / Merged
	Deleted,
	LocallyDeleted, // Missing
	Changed, // Locally Changed but not CheckedOut
	Conflicted,
	Private, // "Not Controlled"/"Not In Depot"/"Untracked"
};

class FPlasticSourceControlState : public ISourceControlState
{
public:
	explicit FPlasticSourceControlState(FString&& InLocalFilename)
		: LocalFilename(MoveTemp(InLocalFilename))
	{
	}

	FPlasticSourceControlState(FString&& InLocalFilename, EWorkspaceState InWorkspaceState)
		: LocalFilename(MoveTemp(InLocalFilename))
		, WorkspaceState(InWorkspaceState)
	{
	}

	FPlasticSourceControlState() = delete;
	FPlasticSourceControlState(const FPlasticSourceControlState& InState) = delete;
	const FPlasticSourceControlState& operator=(const FPlasticSourceControlState& InState) = delete;

	FPlasticSourceControlState(FPlasticSourceControlState&& InState)
	{
		Move(MoveTemp(InState));
	}

	// Comparison operator designed to detect and report only meaningful changes to the Editor, mainly for the purpose of updating Content Browser overlay icons
	bool operator==(const FPlasticSourceControlState& InState) const
	{
		return (WorkspaceState == InState.WorkspaceState)
			&& (LockedBy == InState.LockedBy)
			&& (RetainedBy == InState.RetainedBy)
			&& (IsCurrent() == InState.IsCurrent());
	}
	bool operator!=(const FPlasticSourceControlState& InState) const
	{
		return !(*this == InState);
	}

	const FPlasticSourceControlState& operator=(FPlasticSourceControlState&& InState)
	{
		Move(MoveTemp(InState));
		return *this;
	}

	void Move(FPlasticSourceControlState&& InState)
	{
		if (InState.History.Num() > 0)
		{
			History = MoveTemp(InState.History);
		}
		LocalFilename = MoveTemp(InState.LocalFilename);
		WorkspaceState = InState.WorkspaceState;
		PendingResolveInfo = MoveTemp(InState.PendingResolveInfo);
		PendingMergeParameters = MoveTemp(InState.PendingMergeParameters);
		// Update "fileinfo" information only if the command was issued
		// Don't override "fileinfo" information in case of an optimized/lightweight "whole folder status" triggered by a global Submit Content or Refresh
		if (InState.DepotRevisionChangeset != INVALID_REVISION)
		{
			LockedBy = MoveTemp(InState.LockedBy);
			LockedWhere = MoveTemp(InState.LockedWhere);
			LockedBranch = MoveTemp(InState.LockedBranch);
			LockedId = MoveTemp(InState.LockedId);
			LockedDate = MoveTemp(InState.LockedDate);
			RetainedBy = MoveTemp(InState.RetainedBy);
			RepSpec = MoveTemp(InState.RepSpec);
			DepotRevisionChangeset = InState.DepotRevisionChangeset;
			LocalRevisionChangeset = InState.LocalRevisionChangeset;

			HeadBranch = MoveTemp(InState.HeadBranch);
			HeadAction = MoveTemp(InState.HeadAction);
			HeadChangeList = MoveTemp(InState.HeadChangeList);
			HeadUserName = MoveTemp(InState.HeadUserName);
			HeadModTime = MoveTemp(InState.HeadModTime);
		}
		MovedFrom = MoveTemp(InState.MovedFrom);
		TimeStamp = InState.TimeStamp;

		// Update Revision's pointer to the new State instance
		for (const auto& Revision : History)
		{
			Revision->State = this;
		}
	}

	// debug log utility
	const TCHAR* ToString() const;

	FText ToText() const;

	void PopulateSearchString(TArray<FString>& OutStrings) const
	{
		OutStrings.Emplace(LocalFilename);
	}

	/** ISourceControlState interface */
	virtual int32 GetHistorySize() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetHistoryItem(int32 HistoryIndex) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision(int32 RevisionNumber) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision(const FString& InRevision) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetCurrentRevision() const override;
	virtual FResolveInfo GetResolveInfo() const override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual FSlateIcon GetIcon() const override;
#endif // SOURCE_CONTROL_WITH_SLATE
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayTooltip() const override;
	virtual const FString& GetFilename() const override;
	virtual const FDateTime& GetTimeStamp() const override;
	virtual bool CanCheckIn() const override;
	virtual bool CanCheckout() const override;
	virtual bool IsCheckedOut() const override;
	virtual bool IsCheckedOutOther(FString* Who = nullptr) const override;
	virtual bool IsCheckedOutInOtherBranch(const FString& CurrentBranch = FString()) const override;
	virtual bool IsModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override;
	virtual bool IsCheckedOutOrModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override { return IsCheckedOutInOtherBranch(CurrentBranch) || IsModifiedInOtherBranch(CurrentBranch); }
	virtual TArray<FString> GetCheckedOutBranches() const override { return TArray<FString>(); }
	virtual FString GetOtherUserBranchCheckedOuts() const override { return FString(); }
	virtual bool GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const override;
	virtual bool IsCurrent() const override;
	virtual bool IsSourceControlled() const override;
	virtual bool IsAdded() const override;
	virtual bool IsDeleted() const override;
	virtual bool IsIgnored() const override;
	virtual bool CanEdit() const override;
	virtual bool IsUnknown() const override;
	virtual bool IsModified() const override;
	virtual bool CanAdd() const override;
	virtual bool CanDelete() const override;
	virtual bool IsConflicted() const override;
	virtual bool CanRevert() const override;

	bool IsPendingChanges() const;
	bool IsCheckedOutImplementation() const;
	bool IsLocked() const;
	bool IsRetainedInOtherBranch() const;

public:
	/** History of the item, if any */
	TPlasticSourceControlHistory History;

	/** Filename on disk */
	FString LocalFilename;

	/** Depot and Server info (in the form repo@server:port) */
	FString RepSpec;

	/** Pending rev info with which a file must be resolved, invalid if no resolve pending */
	FResolveInfo PendingResolveInfo;

	/** Unity Version Control Parameters of the merge in progress */
	TArray<FString> PendingMergeParameters;

	/** If a user (another or ourself) has this file locked, this contains their name. */
	FString LockedBy;

	/** Location (Workspace) where the file was exclusively checked-out. */
	FString LockedWhere;

	/** Branch where the file was Locked or is Retained. */
	FString LockedBranch;

	/** Item id of the locked file (for an admin to unlock it). */
	int32 LockedId = INVALID_REVISION;

	/** Date when the file was Locked. */
	FDateTime LockedDate = 0;

	/** If a user (another or ourself) has this file Retained on another branch, this contains their name. */
	FString RetainedBy;

	/** State of the workspace */
	EWorkspaceState WorkspaceState = EWorkspaceState::Unknown;

	/** Latest revision number of the file in the depot (on the current branch) */
	int32 DepotRevisionChangeset = INVALID_REVISION;

	/** Latest revision number at which a file was synced to before being edited */
	int32 LocalRevisionChangeset = INVALID_REVISION;

	/** Original name in case of a Moved/Renamed file */
	FString MovedFrom;

	/** Changelist containing this file */
	FPlasticSourceControlChangelist Changelist;

	/** The timestamp of the last update */
	FDateTime TimeStamp = 0;

	/** The branch with the head change list */
	FString HeadBranch;

	/** The type of action of the last modification */
	FString HeadAction;

	/** The user of the last modification */
	FString HeadUserName;

	/** The last file modification time */
	int64 HeadModTime;

	/** The change list of the last modification */
	int32 HeadChangeList;
};

typedef TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStateRef;
typedef TSharedPtr<FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStatePtr;
