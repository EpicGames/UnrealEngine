// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlUtils.h"

#include "PlasticSourceControlBranch.h"
#include "PlasticSourceControlChangeset.h"
#include "PlasticSourceControlCommand.h"
#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlParsers.h"
#include "PlasticSourceControlProjectSettings.h"
#include "PlasticSourceControlProvider.h"
#include "PlasticSourceControlSettings.h"
#include "PlasticSourceControlShell.h"
#include "PlasticSourceControlState.h"
#include "PlasticSourceControlVersions.h"
#include "ISourceControlModule.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "SoftwareVersion.h"
#include "ScopedTempFile.h"

#include "PlasticSourceControlChangelistState.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/WindowsPlatformMisc.h"
#undef GetUserName
#endif

namespace PlasticSourceControlUtils
{

#define FILE_STATUS_SEPARATOR TEXT(";")

// Run a command and return the result as raw strings
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors)
{
	return PlasticSourceControlShell::RunCommand(InCommand, InParameters, InFiles, OutResults, OutErrors);
}

// Run a command with basic parsing or results & errors from the cm command line process
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	FString Results;
	FString Errors;

	const bool bResult = PlasticSourceControlShell::RunCommand(InCommand, InParameters, InFiles, Results, Errors);

	if (!Results.IsEmpty())
	{
		TArray<FString> ParsedResults;
		Results.ParseIntoArray(ParsedResults, PlasticSourceControlShell::pchDelim, true);
		OutResults.Append(MoveTemp(ParsedResults));
	}
	if (!Errors.IsEmpty())
	{
		TArray<FString> ParsedErrors;
		Errors.ParseIntoArray(ParsedErrors, PlasticSourceControlShell::pchDelim, true);
		OutErrorMessages.Append(MoveTemp(ParsedErrors));
	}

	return bResult;
}

FString FindPlasticBinaryPath()
{
#if PLATFORM_WINDOWS
	return FString(TEXT("cm"));
#else
	return FString(TEXT("/usr/bin/cm"));
#endif
}

static FString FindDesktopApplicationPath()
{
	FString DesktopAppPath;

#if PLATFORM_WINDOWS
	// On Windows, use the registry to find the install location
	FString InstallLocation = TEXT("C:/Program Files/PlasticSCM5");
	if (FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Unity Software Inc.\\Unity DevOps Version Control"), TEXT("Location"), InstallLocation))
	{
		FPaths::NormalizeDirectoryName(InstallLocation);
	}

	const TCHAR* PlasticExe = TEXT("client/plastic.exe");
	const TCHAR* GluonExe = TEXT("client/gluon.exe");
	DesktopAppPath = FPaths::Combine(InstallLocation, FPlasticSourceControlModule::Get().GetProvider().IsPartialWorkspace() ? GluonExe : PlasticExe);
#elif PLATFORM_MAC
	const TCHAR* PlasticExe = TEXT("/Applications/PlasticSCM.app/Contents/MacOS/macplasticx");
	const TCHAR* GluonExe = TEXT("/Applications/Gluon.app/Contents/MacOS/macgluonx");
	DesktopAppPath = FPlasticSourceControlModule::Get().GetProvider().IsPartialWorkspace() ? GluonExe : PlasticExe;
#elif PLATFORM_LINUX
	const TCHAR* PlasticExe = TEXT("/usr/bin/plasticgui");
	const TCHAR* GluonExe = TEXT("/usr/bin/gluon");
	DesktopAppPath = FPlasticSourceControlModule::Get().GetProvider().IsPartialWorkspace() ? GluonExe : PlasticExe;
#endif

	return DesktopAppPath;
}

static bool OpenDesktopApplication(const FString& InCommandLineArguments)
{
	const FString DesktopAppPath = FindDesktopApplicationPath();

	UE_LOGF(LogSourceControl, Log, "Opening the Desktop application (%ls %ls)", *DesktopAppPath, *InCommandLineArguments);

	FProcHandle Proc = FPlatformProcess::CreateProc(*DesktopAppPath, *InCommandLineArguments, true, false, false, nullptr, 0, nullptr, nullptr, nullptr);
	if (!Proc.IsValid())
	{
		UE_LOGF(LogSourceControl, Error, "Opening the Desktop application (%ls %ls) failed.", *DesktopAppPath, *InCommandLineArguments);
		FPlatformProcess::CloseProc(Proc);
		return false;
	}

	return true;
}

bool OpenDesktopApplication(const bool bInBranchExplorer)
{
	const FString CommandLineArguments = FString::Printf(TEXT("--wk=\"%s\" %s"),
		*FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot(),
		bInBranchExplorer ? TEXT("--view=BranchExplorerView") : TEXT(""));

	return OpenDesktopApplication(CommandLineArguments);
}

static FString GetFullSpec(const int32 InChangesetId)
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	return FString::Printf(TEXT("cs:%d@%s@%s"), InChangesetId, *Provider.GetRepositoryName(), *Provider.GetServerUrl());
}

static FString GetFullSpec(const FString& InBranchName)
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	return FString::Printf(TEXT("br:%s@%s@%s"), *InBranchName, *Provider.GetRepositoryName(), *Provider.GetServerUrl());
}

bool OpenDesktopApplicationForDiff(const int32 InChangesetId)
{
	const FString CommandLineArguments = FString::Printf(TEXT("--diffchangeset=\"%s\""),
		*GetFullSpec(InChangesetId));

	return OpenDesktopApplication(CommandLineArguments);
}

bool OpenDesktopApplicationForDiff(const int32 InChangesetIdSrc, const int32 InChangesetIdDst)
{
	const FString CommandLineArguments = FString::Printf(TEXT("--diffchangesetsrc=\"%s\" --diffchangesetdst=\"%s\""),
		*GetFullSpec(InChangesetIdSrc), *GetFullSpec(InChangesetIdDst));

	return OpenDesktopApplication(CommandLineArguments);
}

bool OpenDesktopApplicationForDiff(const FString& InBranchName)
{
	const FString CommandLineArguments = FString::Printf(TEXT("--diffbranch=\"%s\""),
		*GetFullSpec(InBranchName));

	return OpenDesktopApplication(CommandLineArguments);
}

void OpenLockRulesInCloudDashboard(const FString& InOrganizationName)
{
	const FString OrganizationLockRulesURL = FString::Printf(
		TEXT("https://dashboard.unity3d.com/devops/organizations/default/plastic-scm/organizations/%s/lock-rules"),
		*InOrganizationName
	);
	FPlatformProcess::LaunchURL(*OrganizationLockRulesURL, NULL, NULL);
}

// Find the root of the workspace, looking from the provided path and upward in its parent directories.
bool GetWorkspacePath(const FString& InPath, FString& OutWorkspaceRoot)
{
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--format={wkpath}"));
	Parameters.Add(TEXT("."));
	const bool bFound = RunCommand(TEXT("getworkspacefrompath"), Parameters, TArray<FString>(), Results, ErrorMessages);
	if (bFound && Results.Num() > 0)
	{
		OutWorkspaceRoot = MoveTemp(Results[0]);
		FPaths::NormalizeDirectoryName(OutWorkspaceRoot);
		OutWorkspaceRoot.AppendChar(TEXT('/'));
	}
	else
	{
		OutWorkspaceRoot = InPath; // If not found, return the provided dir as best possible root.
	}
	return bFound;
}

// This is called once by FPlasticSourceControlProvider::CheckPlasticAvailability()
bool GetPlasticScmVersion(FSoftwareVersion& OutPlasticScmVersion)
{
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	const bool bResult = RunCommand(TEXT("version"), TArray<FString>(), TArray<FString>(), Results, ErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		OutPlasticScmVersion = FSoftwareVersion(MoveTemp(Results[0]));
		return true;
	}
	return false;
}

bool GetCmLocation(FString& OutCmLocation)
{
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	const bool bResult = RunCommand(TEXT("location"), TArray<FString>(), TArray<FString>(), Results, ErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		OutCmLocation = MoveTemp(Results[0]);
		return true;
	}
	return false;
}

bool GetConfigSetFilesAsReadOnly()
{
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("setfileasreadonly"));
	const bool bResult = RunCommand(TEXT("getconfig"), Parameters, TArray<FString>(), Results, ErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		if ((Results[0].Compare(TEXT("yes"), ESearchCase::IgnoreCase) == 0) || (Results[0].Compare(TEXT("true"), ESearchCase::IgnoreCase) == 0))
		{
			return true;
		}
	}
	return false;
}

FString GetConfigDefaultRepServer()
{
	FString ServerUrl;
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("defaultrepserver"));
	const bool bResult = RunCommand(TEXT("getconfig"), Parameters, TArray<FString>(), Results, ErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		ServerUrl = MoveTemp(Results[0]);
	}
	return ServerUrl;
}

FString GetDefaultUserName()
{
	FString UserName;
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	const bool bResult = RunCommand(TEXT("whoami"), TArray<FString>(), TArray<FString>(), Results, ErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		UserName = MoveTemp(Results[0]);
	}

	return UserName;
}

FString GetProfileUserName(const FString& InServerUrl)
{
	FString UserName;
	TArray<FString> Results;
	TArray<FString> Parameters;
	TArray<FString> ErrorMessages;
	Parameters.Add("list");
	Parameters.Add(TEXT("--format=\"{server};{user}\""));
	bool bResult = RunCommand(TEXT("profile"), Parameters, TArray<FString>(), Results, ErrorMessages);
	if (bResult)
	{
		bResult = PlasticSourceControlParsers::ParseProfileInfo(Results, InServerUrl, UserName);
	}

	return UserName;
}

bool GetWorkspaceName(const FString& InWorkspaceRoot, FString& OutWorkspaceName, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Results;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--format={wkname}"));
	TArray<FString> Files;
	Files.Add(InWorkspaceRoot); // Uses an absolute path so that the error message is explicit
	// Get the workspace name
	const bool bResult = RunCommand(TEXT("getworkspacefrompath"), Parameters, Files, Results, OutErrorMessages);
	if (bResult && Results.Num() > 0)
	{
		// NOTE: an old version of cm getworkspacefrompath didn't return an error code so we had to rely on the error message
		if (!Results[0].EndsWith(TEXT(" is not in a workspace.")))
		{
			OutWorkspaceName = MoveTemp(Results[0]);
		}
	}

	return bResult;
}

bool GetWorkspaceInfo(FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Results;
	bool bResult = RunCommand(TEXT("workspaceinfo"), TArray<FString>(), TArray<FString>(), Results, OutErrorMessages);
	if (bResult)
	{
		bResult = PlasticSourceControlParsers::ParseWorkspaceInfo(Results, OutWorkspaceSelector, OutBranchName, OutRepositoryName, OutServerUrl);
	}

	return bResult;
}

bool GetChangesetNumber(int32& OutChangesetNumber, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Results;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--header"));
	Parameters.Add(TEXT("--machinereadable"));
	Parameters.Add(TEXT("--fieldseparator=\"") FILE_STATUS_SEPARATOR TEXT("\""));
	bool bResult = RunCommand(TEXT("status"), Parameters, TArray<FString>(), Results, OutErrorMessages);
	if (bResult)
	{
		bResult = PlasticSourceControlParsers::GetChangesetFromWorkspaceStatus(Results, OutChangesetNumber);
	}

	return bResult;
}

bool RunCheckConnection(FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl, TArray<FString>& OutInfoMessages, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Parameters;
	if (PlasticSourceControlUtils::GetWorkspaceInfo(OutWorkspaceSelector, OutBranchName, OutRepositoryName, OutServerUrl, OutErrorMessages))
	{
		Parameters.Add(FString::Printf(TEXT("--server=%s"), *OutServerUrl));
	}
	return PlasticSourceControlUtils::RunCommand(TEXT("checkconnection"), Parameters, TArray<FString>(), OutInfoMessages, OutErrorMessages);
}

FString UserNameToDisplayName(const FString& InUserName)
{
	if (const FString* Result = GetDefault<UPlasticSourceControlProjectSettings>()->UserNameToDisplayName.Find(InUserName))
	{
		return *Result;
	}
	else if (GetDefault<UPlasticSourceControlProjectSettings>()->bHideEmailDomainInUsername)
	{
		int32 EmailDomainSeparatorIndex;
		if (InUserName.FindChar(TEXT('@'), EmailDomainSeparatorIndex))
		{
			return InUserName.Left(EmailDomainSeparatorIndex);
		}
	}

	return InUserName;
}

/**
 * @brief Run a "status" command for a directory to get the local workspace file states
 *
 *  ie. Changed, CheckedOut, Copied, Replaced, Added, Private, Ignored, Deleted, LocallyDeleted, Moved, LocallyMoved
 *
 *  It is either a command for a whole directory (ie. "Content/", in case of "Submit to Source Control"),
 * or for one or more files all on a same directory (by design, since we group files by directory in RunUpdateStatus())
 *
 * @param[in]	InDir				The path to the common directory of all the files listed after (never empty).
 * @param[in]	InFiles				List of files in a directory, or the path to the directory itself (never empty).
 * @param[in]	InSearchType		Call "status" with "--all", or with just "--controlledchanged" when doing only a quick check following a source control operation
 * @param[out]	OutErrorMessages	Error messages from the "status" command
 * @param[out]	OutStates			States of files for witch the status has been gathered (distinct than InFiles in case of a "directory status")
 * @param[out]	OutChangeset		The current Changeset Number
 */
static bool RunStatus(const FString& InDir, TArray<FString>&& InFiles, const EStatusSearchType InSearchType, TArray<FString>& OutErrorMessages, TArray<FPlasticSourceControlState>& OutStates, int32& OutChangeset)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunStatus);

	check(InFiles.Num() > 0);

	TArray<FString> Parameters;
	Parameters.Add(TEXT("--machinereadable"));
	Parameters.Add(TEXT("--fieldseparator=\"") FILE_STATUS_SEPARATOR TEXT("\""));
	Parameters.Add(TEXT("--controlledchanged"));
	if (InSearchType == EStatusSearchType::All)
	{
		// NOTE: don't use "--all" to avoid searching for --localmoved since it's the most time consuming (beside --changed)
		// and its not used by the plugin (matching similarities doesn't seem to work with .uasset files)
		// TODO: add a user settings to avoid searching for --changed and --localdeleted, to work like Perforce on big projects,
		// provided that the user understands the consequences (they won't see assets modified locally without a proper checkout)
		Parameters.Add(TEXT("--changed"));
		Parameters.Add(TEXT("--localdeleted"));
		Parameters.Add(TEXT("--private"));
		Parameters.Add(TEXT("--ignored"));
	}

	// If the version of cm is recent enough use the new --iscochanged for "CO+CH" status
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const bool bUsesCheckedOutChanged = Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::StatusIsCheckedOutChanged;
	if (bUsesCheckedOutChanged)
	{
		Parameters.Add(TEXT("--iscochanged"));
	}

	// "cm status" only operate on one path (file or directory) at a time, so use one common path for multiple files in a directory
	TArray<FString> OnePath;
	// Only one file: optim very useful for the .uproject file at the root to avoid parsing the whole repository
	// (but doesn't work if the file is deleted)
	const bool bSingleFile = (InFiles.Num() == 1) && (FPaths::FileExists(InFiles[0]));
	if (bSingleFile)
	{
		OnePath.Add(InFiles[0]);
	}
	else
	{
		OnePath.Add(InDir);
	}
	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	const bool bResult = RunCommand(TEXT("status"), Parameters, OnePath, Results, ErrorMessages);
	OutErrorMessages.Append(MoveTemp(ErrorMessages));
	if (bResult)
	{
		// Parse the first line of status with the Changeset number, then remove it to work on a plain list of files
		if (Results.Num() > 0)
		{
			PlasticSourceControlParsers::GetChangesetFromWorkspaceStatus(Results, OutChangeset);
			Results.RemoveAt(0, EAllowShrinking::No);
		}

		// Normalize file paths in the result (convert all '\' to '/')
		for (FString& Result : Results)
		{
			FPaths::NormalizeFilename(Result);
		}

		const bool bWholeDirectory = (InFiles.Num() == 1) && (InFiles[0] == InDir);
		if (bWholeDirectory)
		{
			// 1) Special case for "status" of a directory: requires a specific parse logic.
			//   (this is triggered by the "Submit to Source Control" top menu button, but also for the initial check, the global Revert etc)
			UE_LOGF(LogSourceControl, Verbose, "RunStatus(%ls): 1) special case for status of a directory:", *InDir);
			PlasticSourceControlParsers::ParseDirectoryStatusResult(InDir, Results, OutStates);
		}
		else
		{
			// 2) General case for one or more files in the same directory.
			UE_LOGF(LogSourceControl, Verbose, "RunStatus(%ls...): 2) general case for %d file(s) in a directory (%ls)", *InFiles[0], InFiles.Num(), *InDir);
			PlasticSourceControlParsers::ParseFileStatusResult(MoveTemp(InFiles), Results, OutStates);
		}
	}

	return bResult;
}

// Cache Locks with a Timestamp, and an InvalidateCachedLocks() function
class FLocksCache
{
public:
	void Reset()
	{
		FScopeLock Lock(&CriticalSection);
		Locks.Reset();
		Timestamp = FDateTime();
	}

	void SetLocks(const TArray<FPlasticSourceControlLockRef>& InLocks)
	{
		FScopeLock Lock(&CriticalSection);
		Locks = InLocks;
		Timestamp = FDateTime::Now();
	}

	bool GetLocks(TArray<FPlasticSourceControlLockRef>& OutLocks)
	{
		FScopeLock Lock(&CriticalSection);
		const FTimespan ElapsedTime = FDateTime::Now() - Timestamp;
		if (ElapsedTime.GetTotalSeconds() < 60.0)
		{
			OutLocks = Locks;
			return true;
		}
		return false;
	}

private:
	TArray<FPlasticSourceControlLockRef> Locks;
	FDateTime Timestamp;
	FCriticalSection CriticalSection;
};

static FLocksCache LocksCacheForAllDestBranches;
static FLocksCache LocksCacheForWorkingBranch;

void InvalidateLocksCache()
{
	LocksCacheForAllDestBranches.Reset();
	LocksCacheForWorkingBranch.Reset();
}

bool RunListLocks(const FPlasticSourceControlProvider& InProvider, const bool bInForAllDestBranches, TArray<FPlasticSourceControlLockRef>& OutLocks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunListLocks);

	FLocksCache& LocksCache = bInForAllDestBranches ? LocksCacheForAllDestBranches : LocksCacheForWorkingBranch;

	if (LocksCache.GetLocks(OutLocks))
		return true;

	TArray<FString> Results;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("list"));
	Parameters.Add(TEXT("--machinereadable"));
	Parameters.Add(TEXT("--smartlocks"));
	Parameters.Add(FString::Printf(TEXT("--repository=%s"), *InProvider.GetRepositoryName()));
	Parameters.Add(TEXT("--anystatus"));
	Parameters.Add(TEXT("--fieldseparator=\"") FILE_STATUS_SEPARATOR TEXT("\""));
	// NOTE: --dateformat was added to smartlocks a couple of releases later in version 11.0.16.8133
	Parameters.Add(TEXT("--dateformat=yyyy-MM-ddTHH:mm:ss"));
	// For displaying Locks as a status overlay icon in the Content Browser, restricts the Locks to only those applying to the current branch so there can be only one and never any ambiguity
	if (!bInForAllDestBranches && (InProvider.GetPlasticScmVersion() >= PlasticSourceControlVersions::WorkingBranch))
	{
		// Note: here is one of the rare places where we need to use a branch name, not a workspace selector
		Parameters.Add(FString::Printf(TEXT("--workingbranch=%s"), *InProvider.GetBranchName()));
	}
	const bool bResult = RunCommand(TEXT("lock"), Parameters, TArray<FString>(), Results, ErrorMessages);

	if (bResult)
	{
		OutLocks.Reserve(Results.Num());
		for (int32 IdxResult = 0; IdxResult < Results.Num(); IdxResult++)
		{
			const FString& Result = Results[IdxResult];
			FPlasticSourceControlLock&& Lock = PlasticSourceControlParsers::ParseLockInfo(Result);
			OutLocks.Add(MakeShareable(new FPlasticSourceControlLock(Lock)));
		}

		LocksCache.SetLocks(OutLocks);
	}

	return bResult;
}

TArray<FPlasticSourceControlLockRef> GetLocksForWorkingBranch(const FPlasticSourceControlProvider& InProvider, const TArray<FString>& InFiles)
{
	TArray<FPlasticSourceControlLockRef> Locks;

	// Only get locks for the current working branch
	const bool bInForAllDestBranches = false;
	RunListLocks(InProvider, bInForAllDestBranches, Locks);

	TArray<FPlasticSourceControlLockRef> MatchingLocks;
	MatchingLocks.Reserve(InFiles.Num());

	// Only return locks for the specified files
	for (const FString& File : InFiles)
	{
		for (const FPlasticSourceControlLockRef& Lock : Locks)
		{
			if (File.EndsWith(Lock->Path))
			{
				MatchingLocks.Add(Lock);
				break;
			}
		}
	}

	return MatchingLocks;
}

TArray<FString> LocksToFileNames(const FString InWorkspaceRoot, const TArray<FPlasticSourceControlLockRef>& InLocks)
{
	TArray<FString> Files;

	// Note: remove the slash '/' from the end of the Workspace root to Combine it with server paths also starting with a slash
	const FString& WorkspaceRoot = InWorkspaceRoot[InWorkspaceRoot.Len() - 1] == TEXT('/') ? InWorkspaceRoot.LeftChop(1) : InWorkspaceRoot;

	Files.Reserve(InLocks.Num());
	for (const FPlasticSourceControlLockRef& Lock : InLocks)
	{
		Files.AddUnique(FPaths::Combine(WorkspaceRoot, Lock->Path));
	}

	return Files;
}

/**
 * @brief Run a "fileinfo" command to update complementary status information of given files.
 *
 * ie RevisionChangeset, RevisionHeadChangeset, RepSpec, LockedBy, LockedWhere
 *
 * @param[in]		bInWholeDirectory	If executed on a whole directory (typically Content/) for a "Submit Content" operation, optimize fileinfo more aggressively
 * @param			bInUpdateHistory	If getting the history of files, force execute the fileinfo command required to get RepSpec of XLinks (history view or visual diff)
 * @param[out]		OutErrorMessages	Error messages from the "fileinfo" command
 * @param[in,out]	InOutStates			List of file states in the directory, gathered by the "status" command, completed by results of the "fileinfo" command
 */
static bool RunFileinfo(const bool bInWholeDirectory, const bool bInUpdateHistory, TArray<FString>& OutErrorMessages, TArray<FPlasticSourceControlState>& InOutStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunFileinfo);

	bool bResult = true;
	TArray<FString> SelectedFiles;

	TArray<FPlasticSourceControlState> SelectedStates;
	TArray<FPlasticSourceControlState> OptimizedStates;
	for (FPlasticSourceControlState& State : InOutStates)
	{
		// 1) Issue a "fileinfo" command for controlled files (to know if they are up to date and can be checked-out or checked-in)
		// but only if controlled unchanged, or locally changed / locally deleted,
		// optimizing for files that are CheckedOut/Added/Deleted/Moved/Copied/Replaced/NotControled/Ignored/Private/Unknown
		// (since there is no point to check if they are up to date in these cases; they are already checked-out or not controlled).
		// This greatly reduce the time needed to do some operations like "Add" or "Move/Rename/Copy" when there is some latency with the server (eg cloud).
		//
		// 2) bInWholeDirectory: In the case of a "whole directory status" triggered by the "Submit Content" operation,
		// don't even issue a "fileinfo" command for unchanged Controlled files since they won't be considered them for submit.
		// This greatly reduce the time needed to open the Submit window.
		//
		// 3) bInUpdateHistory: When the plugin needs to update the history of files, it needs to know if it's on a XLink,
		// so the fileinfo command is required here to get the RepSpec
		if (bInUpdateHistory
			|| ((State.WorkspaceState == EWorkspaceState::Controlled) && !bInWholeDirectory)
			||	(State.WorkspaceState == EWorkspaceState::Changed)
			||	(State.WorkspaceState == EWorkspaceState::LocallyDeleted)
			)
		{
			SelectedFiles.Add(State.LocalFilename);
			SelectedStates.Add(MoveTemp(State));
		}
		else
		{
			OptimizedStates.Add(MoveTemp(State));
		}
	}
	InOutStates = MoveTemp(OptimizedStates);

	if (SelectedStates.Num())
	{
		TArray<FString> Results;
		TArray<FString> ErrorMessages;
		TArray<FString> Parameters;
		Parameters.Add(TEXT("--format=\"{RevisionChangeset};{RevisionHeadChangeset};{RepSpec};{LockedBy};{LockedWhere};{ServerPath}\""));
		bResult = RunCommand(TEXT("fileinfo"), Parameters, SelectedFiles, Results, ErrorMessages);
		OutErrorMessages.Append(MoveTemp(ErrorMessages));
		if (bResult)
		{
			PlasticSourceControlParsers::ParseFileinfoResults(Results, SelectedStates);
			InOutStates.Append(MoveTemp(SelectedStates));
		}
	}

	return bResult;
}

// Check if merging, and from which changelist, then execute a cm merge command to amend status for listed files
static bool RunCheckMergeStatus(const TArray<FString>& InFiles, TArray<FString>& OutErrorMessages, TArray<FPlasticSourceControlState>& OutStates)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunCheckMergeStatus);

	bool bResult = false;
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();

	const FString MergeProgressFilename = FPaths::Combine(*Provider.GetPathToWorkspaceRoot(), TEXT(".plastic/plastic.mergeprogress"));
	if (FPaths::FileExists(MergeProgressFilename))
	{
		// read in file as string
		FString MergeProgressContent;
		if (FFileHelper::LoadFileToString(MergeProgressContent, *MergeProgressFilename))
		{
			UE_LOGF(LogSourceControl, Verbose, "RunCheckMergeStatus: %ls:\n%ls", *MergeProgressFilename, *MergeProgressContent);
			// Content is in one line, looking like the following:
			// Target: mount:56e62dd7-241f-41e9-8c6b-dd4ca4513e62#/#UEMergeTest@localhost:8087 merged from: Merge 4
			// Target: mount:56e62dd7-241f-41e9-8c6b-dd4ca4513e62#/#UEMergeTest@localhost:8087 merged from: Cherrypicking 3
			// Target: mount:56e62dd7-241f-41e9-8c6b-dd4ca4513e62#/#UEMergeTest@localhost:8087 merged from: IntervalCherrypick 2 4
			// 1) Extract the word after "merged from: "
			static const FString MergeFromString(TEXT("merged from: "));
			const int32 MergeFromIndex = MergeProgressContent.Find(MergeFromString, ESearchCase::CaseSensitive);
			if (MergeFromIndex > INDEX_NONE)
			{
				const FString MergeType = MergeProgressContent.RightChop(MergeFromIndex + MergeFromString.Len());
				int32 SpaceBeforeChangesetIndex;
				if (MergeType.FindChar(TEXT(' '), SpaceBeforeChangesetIndex))
				{
					// 2) In case of "Merge" or "Cherrypicking" extract the merge changelist xxx after the last space (use case for merge from "branch", from "label", and for "merge on Update")
					const FString ChangesetString = MergeType.RightChop(SpaceBeforeChangesetIndex + 1);
					const int32 Changeset = FCString::Atoi(*ChangesetString);
					const FString ChangesetSpecification = FString::Printf(TEXT("cs:%d"), Changeset);

					TArray<FString> Results;
					TArray<FString> ErrorMessages;
					TArray<FString> Parameters;
					Parameters.Add(ChangesetSpecification);

					int32 SpaceBeforeChangeset2Index;
					if (ChangesetString.FindLastChar(TEXT(' '), SpaceBeforeChangeset2Index))
					{
						// 3) In case of "IntervalCherrypick", extract the 2 changelists
						const FString Changeset2String = ChangesetString.RightChop(SpaceBeforeChangeset2Index + 1);
						const int32 Changeset2 = FCString::Atoi(*Changeset2String);
						const FString Changeset2Specification = FString::Printf(TEXT("--interval-origin=cs:%d"), Changeset2);

						Parameters.Add(Changeset2Specification);
					}
					else
					{
						if (MergeType.StartsWith(TEXT("Cherrypicking"), ESearchCase::CaseSensitive))
						{
							Parameters.Add(TEXT("--cherrypicking"));
						}
					}
					// Store the Merge Parameters for reuse with later "Resolve" operation
					const TArray<FString> PendingMergeParameters = Parameters;
					Parameters.Add(TEXT("--machinereadable"));
					// call 'cm merge cs:xxx --machinereadable' (only dry-run, without the --merge parameter)
					bResult = RunCommand(TEXT("merge"), Parameters, TArray<FString>(), Results, ErrorMessages);
					OutErrorMessages.Append(MoveTemp(ErrorMessages));
					// Parse the result, one line for each conflicted files:
					for (const FString& Result : Results)
					{
						PlasticSourceControlParsers::FPlasticMergeConflictParser MergeConflict(Result);
						UE_LOGF(LogSourceControl, Log, "MergeConflict.Filename: '%ls'", *MergeConflict.Filename);
						for (FPlasticSourceControlState& State : OutStates)
						{
							UE_LOGF(LogSourceControl, Log, "State.LocalFilename: '%ls'", *State.LocalFilename);
							if (State.LocalFilename.EndsWith(MergeConflict.Filename, ESearchCase::CaseSensitive))
							{
								UE_LOGF(LogSourceControl, Verbose, "MergeConflict '%ls' found Base cs:%ls From cs:%ls", *MergeConflict.Filename, *MergeConflict.BaseChangeset, *MergeConflict.SourceChangeset);
								State.WorkspaceState = EWorkspaceState::Conflicted;
								State.PendingResolveInfo = {
									MergeConflict.Filename,
									MergeConflict.Filename,
									MergeConflict.SourceChangeset,
									MergeConflict.BaseChangeset
								};
								State.PendingMergeParameters = PendingMergeParameters;
								break;
							}
						}
					}
				}
			}
		}
	}

	return bResult;
}

FString FindCommonDirectory(const FString& InPath1, const FString& InPath2)
{
	const int32 MinLen = FMath::Min(InPath1.Len(), InPath2.Len());
	int32 IndexAfterLastCommonSeparator = 0;
	for (int32 Index = 0; Index < MinLen; Index++)
	{
		if (InPath1[Index] != InPath2[Index])
		{
			break;
		}
		if (InPath1[Index] == TEXT('/'))
		{
			IndexAfterLastCommonSeparator = Index + 1;
		}
	}
	return InPath1.Left(IndexAfterLastCommonSeparator);
}

// Structure to group all files belonging to a root dir, storing their best/longest common directory
struct FFilesInCommonDir
{
	// Best/longest common directory, slash terminated, based on FindCommonDirectory()
	FString			CommonDir;
	TArray<FString>	Files;
};

// Run a batch of Plastic "status" and "fileinfo" commands to update status of given files and directories.
bool RunUpdateStatus(const TArray<FString>& InFiles, const EStatusSearchType InSearchType, const bool bInUpdateHistory, TArray<FString>& OutErrorMessages, TArray<FPlasticSourceControlState>& OutStates, int32& OutChangeset)
{
	bool bResults = true;

	const FString& WorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();

	// The "status" command only operate on one directory-tree at a time (whole tree recursively)
	// not on different folders with no common root.
	// But "Submit to Source Control" ask for the State of many different directories,
	// from Project/Content and Project/Config, Engine/Content, Engine/Plugins/<...>/Content...

	// In a similar way, a checkin can involve files from different subdirectories, and UpdateStatus is called for all of them at once.

	static TArray<FString> RootDirs =
	{
		FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()),
		FPaths::ConvertRelativePathToFull(FPaths::ProjectConfigDir()),
		FPaths::ConvertRelativePathToFull(FPaths::ProjectPluginsDir()),
		FPaths::ConvertRelativePathToFull(FPaths::GameSourceDir()),
		FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir())
	};

	// 1) So here we group files by path (ie. by subdirectory)
	TMap<FString, FFilesInCommonDir> GroupOfFiles;
	for (const FString& File : InFiles)
	{
		// Discard all file/paths that are not under the workspace root (typically excluding the Engine content)
		if (!File.StartsWith(WorkspaceRoot))
		{
			UE_LOGF(LogSourceControl, Verbose, "%ls is out of the Workspace", *File);
			continue;
		}

		bool bDirFound = false;
		for (const auto& RootDir : RootDirs)
		{
			if (File.StartsWith(RootDir))
			{
				FFilesInCommonDir* ExistingGroup = GroupOfFiles.Find(RootDir);
				if (ExistingGroup != nullptr)
				{
					// See if we have to update the CommonDir
					if (!File.StartsWith(ExistingGroup->CommonDir))
					{
						// the file is not in the same path, we need to find their common dir
						ExistingGroup->CommonDir = FindCommonDirectory(ExistingGroup->CommonDir, File);
					}
					ExistingGroup->Files.Add(File);
				}
				else
				{
					FString Path = FPaths::GetPath(File) + TEXT('/');
					GroupOfFiles.Add(RootDir, { MoveTemp(Path), {File}});
				}

				bDirFound = true;
				break;
			}
		}

		// If the file isn't part of our root directories, we simply add its directory as a new group.
		// It means that the group is dedicated to the directory, and as such its CommonDir is the directory itself.
		// This should be an edge case (typically the uproject file) .
		if (!bDirFound)
		{
			const FString Path = FPaths::GetPath(File) + TEXT('/');
			FFilesInCommonDir* ExistingGroup = GroupOfFiles.Find(Path);
			if (ExistingGroup != nullptr)
			{
				ExistingGroup->Files.Add(File);
			}
			else
			{
				GroupOfFiles.Add(Path, { Path, {File} });
			}
		}
	}

	if (InFiles.Num() > 0)
	{
		UE_LOGF(LogSourceControl, Verbose, "RunUpdateStatus: %d file(s)/%d directory(ies) ('%ls'...)", InFiles.Num(), GroupOfFiles.Num(), *InFiles[0]);
	}
	else
	{
		UE_LOGF(LogSourceControl, Warning, "RunUpdateStatus: NO file");
	}

	// 2) then we can batch Plastic status operation by subdirectory
	for (auto& Group : GroupOfFiles)
	{
		const bool bWholeDirectory = ((Group.Value.Files.Num() == 1) && (Group.Value.CommonDir == Group.Value.Files[0]));

		// Run a "status" command on the directory to get workspace file states.
		// (ie. Changed, CheckedOut, Copied, Replaced, Added, Private, Ignored, Deleted, LocallyDeleted, Moved, LocallyMoved)
		TArray<FPlasticSourceControlState> States;
		const bool bGroupOk = RunStatus(Group.Value.CommonDir, MoveTemp(Group.Value.Files), InSearchType, OutErrorMessages, States, OutChangeset);
		if (!bGroupOk)
		{
			bResults = false;
		}
		else if (States.Num() > 0)
		{
			// Run a "fileinfo" command to update complementary status information of given files.
			// (ie RevisionChangeset, RevisionHeadChangeset, RepSpec, LockedBy, LockedWhere, ServerPath)
			// In case of "whole directory status", there is no explicit file in the group (it contains only the directory)
			// => work on the list of files discovered by RunStatus()
			bResults &= RunFileinfo(bWholeDirectory, bInUpdateHistory, OutErrorMessages, States);
		}
		OutStates.Append(MoveTemp(States));
	}

	// Check if merging, and from which changelist, then execute a cm merge command to amend status for listed files
	RunCheckMergeStatus(InFiles, OutErrorMessages, OutStates);

	return bResults;
}

// Run a "getfile" command to dump the binary content of a revision into a file.
bool RunGetFile(const FString& InRevSpec, const FString& InDumpFileName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunGetFile);

	int32	ReturnCode = 0;
	FString Results;
	FString Errors;

	TArray<FString> Parameters;
	Parameters.Add(FString::Printf(TEXT("\"%s\""), *InRevSpec));
	Parameters.Add(TEXT("--raw"));
	Parameters.Add(FString::Printf(TEXT("--file=\"%s\""), *InDumpFileName));
	const bool bResult = PlasticSourceControlUtils::RunCommand(TEXT("getfile"), Parameters, TArray<FString>(), Results, Errors);

	return bResult;
}

// Run a Plastic "history" command and parse it's XML result.
bool RunGetHistory(const bool bInUpdateHistory, TArray<FPlasticSourceControlState>& InOutStates, TArray<FString>& OutErrorMessages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunGetHistory);

	bool bResult = true;
	FString Results;
	FString Errors;
	TArray<FString> Parameters;
	// Detecting move and deletion is costly as it is implemented as two extra queries to the server; do it only when getting the history of the current branch
	if (bInUpdateHistory)
	{
		Parameters.Add(TEXT("--moveddeleted"));
	}
	const FScopedTempFile HistoryResultFile(TEXT("History-"), TEXT(".xml"));
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *HistoryResultFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	if (Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::NewHistoryLimit)
	{
		if (bInUpdateHistory)
		{
			// --limit=0 will not limit the number of revisions, as stated by LimitNumberOfRevisionsInHistory
			Parameters.Add(FString::Printf(TEXT("--limit=%d"), GetDefault<UPlasticSourceControlProjectSettings>()->LimitNumberOfRevisionsInHistory));
		}
		else
		{
			// when only searching for more recent changes on other branches, only the last revision is needed (to compare to the head of the current branch)
			Parameters.Add(TEXT("--limit=1"));
		}
	}

	TArray<FString> Files;
	Files.Reserve(InOutStates.Num());
	for (const FPlasticSourceControlState& State : InOutStates)
	{
		// When getting only the last revision, optimize out if DepotRevisionChangeset is invalid (ie "fileinfo" was optimized out, eg for checked-out files)
		if (!bInUpdateHistory && State.DepotRevisionChangeset == ISourceControlState::INVALID_REVISION)
			continue;

		if (State.IsSourceControlled() && !State.IsAdded())
		{
			Files.Add(State.LocalFilename);
		}
	}
	if (Files.Num() > 0)
	{
		bResult = RunCommand(TEXT("history"), Parameters, Files, Results, Errors);
		if (bResult)
		{
			bResult = PlasticSourceControlParsers::ParseHistoryResults(bInUpdateHistory, HistoryResultFile.GetFilename(), InOutStates);
		}
		if (!Errors.IsEmpty())
		{
			OutErrorMessages.Add(MoveTemp(Errors));
		}
	}

	return bResult;
}

// Run a Plastic "update" command to sync the workspace and parse its XML results.
bool RunUpdate(const TArray<FString>& InFiles, const bool bInIsPartialWorkspace, const FString& InChangesetId, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages)
{
	bool bResult = false;

	TArray<FString> Parameters;
	// Update specified directory to the head of the repository
	// Detect special case for a partial checkout (CS:-1 in Gluon mode)!
	if (!bInIsPartialWorkspace)
	{
		const FScopedTempFile UpdateResultFile(TEXT("Update-"), TEXT(".xml"));
		TArray<FString> InfoMessages;
		if (!InChangesetId.IsEmpty())
		{
			Parameters.Add(FString::Printf(TEXT("--changeset=%s"), *InChangesetId));
		}
		else
		{
			Parameters.Add(TEXT("--last"));
		}
		Parameters.Add(TEXT("--dontmerge"));
		Parameters.Add(TEXT("--noinput"));
		Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *UpdateResultFile.GetFilename()));
		Parameters.Add(TEXT("--encoding=\"utf-8\""));
		bResult = PlasticSourceControlUtils::RunCommand(TEXT("update"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
		if (bResult)
		{
			// Load and parse the result of the update command
			FString Results;
			if (FFileHelper::LoadFileToString(Results, *UpdateResultFile.GetFilename()))
			{
				bResult = PlasticSourceControlParsers::ParseUpdateResults(Results, OutUpdatedFiles);
			}
		}
	}
	else
	{
		TArray<FString> Results;
		if (!InChangesetId.IsEmpty())
		{
			Parameters.Add(FString::Printf(TEXT("--changeset=%s"), *InChangesetId));
		}
		Parameters.Add(TEXT("--report"));
		Parameters.Add(TEXT("--machinereadable"));
		bResult = PlasticSourceControlUtils::RunCommand(TEXT("partial update"), Parameters, InFiles, Results, OutErrorMessages);
		if (bResult)
		{
			bResult = PlasticSourceControlParsers::ParseUpdateResults(Results, OutUpdatedFiles);
		}
	}

	return bResult;
}

// Run a Plastic "status --changelist --xml" and parse its XML result.
bool RunGetChangelists(TArray<FPlasticSourceControlChangelistState>& OutChangelistsStates, TArray<TArray<FPlasticSourceControlState>>& OutCLFilesStates, TArray<FString>& OutErrorMessages)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PlasticSourceControlUtils::RunGetChangelists);

	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();

	FString Results;
	FString Errors;
	const FScopedTempFile GetChangelistFile(TEXT("StatusChangelist-"), TEXT(".xml"));
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--changelists"));
	Parameters.Add(TEXT("--controlledchanged"));
	if (Provider.AccessSettings().GetViewLocalChanges())
	{
		// NOTE: don't use "--all" to avoid searching for --localmoved since it's the most time consuming (beside --changed)
		// and its not used by the plugin (matching similarities doesn't seem to work with .uasset files)
		Parameters.Add(TEXT("--changed"));
		Parameters.Add(TEXT("--localdeleted"));
		Parameters.Add(TEXT("--private"));
	}

	// If the version of cm is recent enough use the new --iscochanged for "CO+CH" status
	const bool bUsesCheckedOutChanged = Provider.GetPlasticScmVersion() >= PlasticSourceControlVersions::StatusIsCheckedOutChanged;
	if (bUsesCheckedOutChanged)
	{
		Parameters.Add(TEXT("--iscochanged"));
	}

	Parameters.Add(TEXT("--noheader"));
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *GetChangelistFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bool bResult = RunCommand(TEXT("status"), Parameters, TArray<FString>(), Results, Errors);
	if (bResult)
	{
		bResult = PlasticSourceControlParsers::ParseChangelistsResults(GetChangelistFile.GetFilename(), OutChangelistsStates, OutCLFilesStates);
	}
	if (!Errors.IsEmpty())
	{
		OutErrorMessages.Add(MoveTemp(Errors));
	}

	return bResult;
}

void AddShelvedFileToChangelist(FPlasticSourceControlChangelistState& InOutChangelistsState, FString&& InFilename, EWorkspaceState InShelveStatus, FString&& InMovedFrom)
{
	TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> ShelveState = MakeShared<FPlasticSourceControlState>(MoveTemp(InFilename), InShelveStatus);
	ShelveState->MovedFrom = MoveTemp(InMovedFrom);

	// Add one revision to be able to fetch the shelved file for diff, if it's not marked for deletion.
	if (InShelveStatus != EWorkspaceState::Deleted)
	{
		const TSharedRef<FPlasticSourceControlRevision, ESPMode::ThreadSafe> SourceControlRevision = MakeShared<FPlasticSourceControlRevision>();
		SourceControlRevision->State = &ShelveState.Get();
		SourceControlRevision->Filename = ShelveState->GetFilename();
		SourceControlRevision->ShelveId = InOutChangelistsState.ShelveId;
		SourceControlRevision->ChangesetNumber = InOutChangelistsState.ShelveId; // Note: for display in the diff window only
		SourceControlRevision->Date = InOutChangelistsState.ShelveDate; // Note: not yet used for display as of UE5.2

		ShelveState->History.Add(SourceControlRevision);
	}

	// In case of a Moved file, it would appear twice in the list, so overwrite it if already in
	if (FSourceControlStateRef* ExistingShelveState = InOutChangelistsState.ShelvedFiles.FindByPredicate(
		[&ShelveState](const FSourceControlStateRef& State)
		{
			return State->GetFilename().Equals(ShelveState->GetFilename());
		}))
	{
		*ExistingShelveState = MoveTemp(ShelveState);
	}
	else
	{
		InOutChangelistsState.ShelvedFiles.Add(MoveTemp(ShelveState));
	}
}

/**
 * Run for each shelve a "diff sh:<ShelveId>" and parse their result to list their files.
 * @param	InOutChangelistsStates	The list of changelists, filled with their shelved files
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetShelveFiles(TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful = true;

	const FString& WorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();

	for (FPlasticSourceControlChangelistState& ChangelistState : InOutChangelistsStates)
	{
		if (ChangelistState.ShelveId != ISourceControlState::INVALID_REVISION)
		{
			TArray<FString> Results;
			TArray<FString> Parameters;
			// TODO switch to custom format --format="{status};{path};{srccmpath}" for better parsing, and perhaps reusing code
			Parameters.Add(FString::Printf(TEXT("sh:%d"), ChangelistState.ShelveId));
			const bool bDiffSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("diff"), Parameters, TArray<FString>(), Results, OutErrorMessages);
			if (bDiffSuccessful)
			{
				bCommandSuccessful = PlasticSourceControlParsers::ParseShelveDiffResult(WorkspaceRoot, MoveTemp(Results), ChangelistState);
			}
		}
	}

	return bCommandSuccessful;
}

// Run find "shelves where owner='me'" and for each shelve matching a changelist a "diff sh:<ShelveId>" and parse their results.
bool RunGetShelves(TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful;

	FString Results;
	FString Errors;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("\"shelves where owner = 'me'\""));
	Parameters.Add(TEXT("--xml")); // TODO convert to -xml=<FScopedTempFile>
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("find"), Parameters, TArray<FString>(), Results, Errors);
	if (bCommandSuccessful)
	{
		bCommandSuccessful = PlasticSourceControlParsers::ParseShelvesResults(Results, InOutChangelistsStates);
		if (bCommandSuccessful)
		{
			bCommandSuccessful = RunGetShelveFiles(InOutChangelistsStates, OutErrorMessages);
		}
	}
	if (!Errors.IsEmpty())
	{
		OutErrorMessages.Add(MoveTemp(Errors));
	}

	return bCommandSuccessful;
}

/**
 * Run for a shelve a "diff sh:<ShelveId> --format='{status};{baserevid};{path}'" and parse the result to list its files.
 * @param	InOutChangelistsStates	The list of changelists, filled with their shelved files
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetShelveFiles(const int32 InShelveId, TArray<FPlasticSourceControlRevision>& OutBaseRevisions, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful = true;

	const FString& WorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();

	if (InShelveId != ISourceControlState::INVALID_REVISION)
	{
		TArray<FString> Results;
		TArray<FString> Parameters;
		Parameters.Add(FString::Printf(TEXT("sh:%d"), InShelveId));
		Parameters.Add(TEXT("--format=\"{status};{baserevid};{path}\""));
		Parameters.Add(TEXT("--encoding=\"utf-8\""));
		const bool bDiffSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("diff"), Parameters, TArray<FString>(), Results, OutErrorMessages);
		if (bDiffSuccessful)
		{
			bCommandSuccessful = PlasticSourceControlParsers::ParseShelveDiffResults(WorkspaceRoot, MoveTemp(Results), OutBaseRevisions);
		}
	}

	return bCommandSuccessful;
}

bool RunGetShelve(const int32 InShelveId, FString& OutComment, FDateTime& OutDate, FString& OutOwner, TArray<FPlasticSourceControlRevision>& OutBaseRevisions, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful;

	FString Results;
	FString Errors;
	TArray<FString> Parameters;
	Parameters.Add(FString::Printf(TEXT("\"shelves where ShelveId = %d\""), InShelveId));
	Parameters.Add(TEXT("--xml")); // TODO convert to -xml=<FScopedTempFile>
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("find"), Parameters, TArray<FString>(), Results, Errors);
	if (bCommandSuccessful)
	{
		bCommandSuccessful = PlasticSourceControlParsers::ParseShelvesResult(Results, OutComment, OutDate, OutOwner);
		if (bCommandSuccessful)
		{
			bCommandSuccessful = RunGetShelveFiles(InShelveId, OutBaseRevisions, OutErrorMessages);
		}
	}
	if (!Errors.IsEmpty())
	{
		OutErrorMessages.Add(MoveTemp(Errors));
	}

	return bCommandSuccessful;
}

bool RunGetChangesets(const FDateTime& InFromDate, TArray<FPlasticSourceControlChangesetRef>& OutChangesets, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful = false;

	const FScopedTempFile ChangesetResultFile(TEXT("FindChangeset-"), TEXT(".xml"));
	TArray<FString> Results;
	TArray<FString> Errors;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("changesets"));
	if (InFromDate != FDateTime())
	{
		Parameters.Add(FString::Printf(TEXT("\"where date >= '%d/%d/%d'\""), InFromDate.GetYear(), InFromDate.GetMonth(), InFromDate.GetDay()));
	}
	Parameters.Add(TEXT("order by ChangesetId desc"));
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *ChangesetResultFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("find"), Parameters, TArray<FString>(), Results, Errors);
	if (bCommandSuccessful && FPaths::FileExists(ChangesetResultFile.GetFilename()))
	{
		bCommandSuccessful = PlasticSourceControlParsers::ParseChangesetsResults(ChangesetResultFile.GetFilename(), OutChangesets);
	}
	if (Errors.Num() > 0)
	{
		OutErrorMessages.Append(MoveTemp(Errors));
	}

	return bCommandSuccessful;
}

bool RunGetChangesetFiles(const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful = false;

	const FScopedTempFile LogResultFile(TEXT("Log-"), TEXT(".xml"));
	FString Results;
	FString Errors;
	TArray<FString> Parameters;
	Parameters.Add(FString::Printf(TEXT("cs:%d"), InChangeset->ChangesetId));
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *LogResultFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("log"), Parameters, TArray<FString>(), Results, Errors);
	if (bCommandSuccessful && FPaths::FileExists(LogResultFile.GetFilename()))
	{
		bCommandSuccessful = PlasticSourceControlParsers::ParseLogResults(LogResultFile.GetFilename(), InChangeset, OutFiles);
	}
	if (!Errors.IsEmpty())
	{
		OutErrorMessages.Add(MoveTemp(Errors));
	}

	return bCommandSuccessful;
}

bool RunGetBranches(const FDateTime& InFromDate, TArray<FPlasticSourceControlBranchRef>& OutBranches, TArray<FString>& OutErrorMessages)
{
	bool bCommandSuccessful;

	const FScopedTempFile BranchResultFile(TEXT("FindBranches-"), TEXT(".xml"));
	FString Results;
	FString Errors;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("branches"));
	if (InFromDate != FDateTime())
	{
		// Find branches created since this date or containing changes dating from or after this date
		Parameters.Add(FString::Printf(TEXT("\"where date >= '%d/%d/%d'\" or changesets >= '%d/%d/%d'"),
			InFromDate.GetYear(), InFromDate.GetMonth(), InFromDate.GetDay(),
			InFromDate.GetYear(), InFromDate.GetMonth(), InFromDate.GetDay()
		));
	}
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *BranchResultFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	bCommandSuccessful = PlasticSourceControlUtils::RunCommand(TEXT("find"), Parameters, TArray<FString>(), Results, Errors);
	if (bCommandSuccessful)
	{
		bCommandSuccessful = PlasticSourceControlParsers::ParseBranchesResults(BranchResultFile.GetFilename(), OutBranches);
	}
	if (!Errors.IsEmpty())
	{
		OutErrorMessages.Add(MoveTemp(Errors));
	}

	return bCommandSuccessful;
}

bool RunSwitch(const FString& InBranchName, const int32 InChangesetId, const bool bInIsPartialWorkspace, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages)
{
	bool bResult = false;

	const FScopedTempFile SwitchResultFile(TEXT("Switch-"), TEXT(".xml"));
	TArray<FString> InfoMessages;
	TArray<FString> Parameters;
	if (InChangesetId != ISourceControlState::INVALID_REVISION)
	{
		// NOTE: not supported by Gluon/partial workspaces
		Parameters.Add(FString::Printf(TEXT("cs:%d"), InChangesetId));
	}
	else
	{
		Parameters.Add(FString::Printf(TEXT("\"br:%s\""), *InBranchName));
	}
	Parameters.Add(TEXT("--noinput"));
	// Detect special case for a partial checkout (CS:-1 in Gluon mode)!
	if (!bInIsPartialWorkspace)
	{
		Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *SwitchResultFile.GetFilename()));
		Parameters.Add(TEXT("--encoding=\"utf-8\""));
		bResult = PlasticSourceControlUtils::RunCommand(TEXT("switch"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
		if (bResult)
		{
			// Load and parse the result of the update command
			FString Results;
			if (FFileHelper::LoadFileToString(Results, *SwitchResultFile.GetFilename()))
			{
				bResult = PlasticSourceControlParsers::ParseUpdateResults(Results, OutUpdatedFiles);
			}
		}
	}
	else
	{
		TArray<FString> Results;
		Parameters.Add(TEXT("--report"));
		bResult = PlasticSourceControlUtils::RunCommand(TEXT("partial switch"), Parameters, TArray<FString>(), Results, OutErrorMessages);
		if (bResult)
		{
			bResult = PlasticSourceControlParsers::ParseUpdateResults(Results, OutUpdatedFiles);
		}
	}

	return bResult;
}

bool RunMergeBranch(const FString& InBranchName, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages)
{
	bool bResult = false;

	const FScopedTempFile MergeResultFile(TEXT("Merge-"), TEXT(".xml"));
	TArray<FString> InfoMessages;
	TArray<FString> Parameters;
	Parameters.Add(FString::Printf(TEXT("--xml=\"%s\""), *MergeResultFile.GetFilename()));
	Parameters.Add(TEXT("--encoding=\"utf-8\""));
	Parameters.Add(TEXT("--merge"));
	Parameters.Add(FString::Printf(TEXT("\"br:%s\""), *InBranchName));
	bResult = PlasticSourceControlUtils::RunCommand(TEXT("merge"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
	if (bResult)
	{
		// Load and parse the result of the update command
		FString Results;
		if (FFileHelper::LoadFileToString(Results, *MergeResultFile.GetFilename()))
		{
			PlasticSourceControlParsers::ParseMergeResults(Results, OutUpdatedFiles);
		}
	}

	return bResult;
}

bool RunCreateBranch(const FString& InBranchName, const FString& InComment, TArray<FString>& OutErrorMessages)
{
	// make a temp file to place our comment message in
	const FScopedTempFile BranchCommentFile(InComment);
	if (!BranchCommentFile.GetFilename().IsEmpty())
	{
		TArray<FString> Parameters;
		TArray<FString> InfoMessages;
		Parameters.Add(TEXT("create"));
		Parameters.Add(FString::Printf(TEXT("\"%s\""), *InBranchName));
		Parameters.Add(FString::Printf(TEXT("--commentsfile=\"%s\""), *BranchCommentFile.GetFilename()));
		return PlasticSourceControlUtils::RunCommand(TEXT("branch"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
	}

	return false;
}

bool RunRenameBranch(const FString& InOldName, const FString& InNewName, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Parameters;
	TArray<FString> InfoMessages;
	Parameters.Add(TEXT("rename"));
	Parameters.Add(FString::Printf(TEXT("\"br:%s\""), *InOldName));
	Parameters.Add(FString::Printf(TEXT("\"%s\""), *InNewName));
	return PlasticSourceControlUtils::RunCommand(TEXT("branch"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
}

bool RunDeleteBranches(const TArray<FString>& InBranchNames, TArray<FString>& OutErrorMessages)
{
	TArray<FString> Parameters;
	TArray<FString> InfoMessages;
	Parameters.Add(TEXT("delete"));
	for (const FString& BranchName : InBranchNames)
	{
		Parameters.Add(FString::Printf(TEXT("\"br:%s\""), *BranchName));
	}
	return PlasticSourceControlUtils::RunCommand(TEXT("branch"), Parameters, TArray<FString>(), InfoMessages, OutErrorMessages);
}

bool UpdateCachedStates(TArray<FPlasticSourceControlState>&& InStates)
{
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const FDateTime Now = FDateTime::Now();

	bool bUpdatedStates = false;
	for (auto&& InState : InStates)
	{
		TSharedRef<FPlasticSourceControlState, ESPMode::ThreadSafe> State = Provider.GetStateInternal(InState.LocalFilename);
		// Only report that the cache was updated if the state changed in a meaningful way, useful to the Editor
		if (*State != InState)
		{
			bUpdatedStates = true;
		}
		*State = MoveTemp(InState);
		State->TimeStamp = Now;
	}

	return bUpdatedStates;
}

void RemoveRedundantErrors(FPlasticSourceControlCommand& InCommand, const FString& InFilter)
{
	bool bFoundRedundantError = false;
	for (const FString& ErrorMessage : InCommand.ErrorMessages)
	{
		if (ErrorMessage.Contains(InFilter, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			InCommand.InfoMessages.Add(ErrorMessage);
			bFoundRedundantError = true;
		}
	}

	InCommand.ErrorMessages.RemoveAll(PlasticSourceControlParsers::FRemoveRedundantErrors(InFilter));

	// if we have no error messages now, assume success!
	if (bFoundRedundantError && InCommand.ErrorMessages.Num() == 0 && !InCommand.bCommandSuccessful)
	{
		InCommand.bCommandSuccessful = true;
	}
}

void SwitchVerboseLogs(const bool bInEnable)
{
	if (bInEnable && LogSourceControl.GetVerbosity() < ELogVerbosity::Verbose)
	{
		LogSourceControl.SetVerbosity(ELogVerbosity::Verbose);
	}
	else if (!bInEnable && LogSourceControl.GetVerbosity() == ELogVerbosity::Verbose)
	{
		LogSourceControl.SetVerbosity(ELogVerbosity::Log);
	}
}

} // namespace PlasticSourceControlUtils

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif