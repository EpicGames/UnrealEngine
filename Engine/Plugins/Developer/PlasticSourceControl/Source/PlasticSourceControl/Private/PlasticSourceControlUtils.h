// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "PlasticSourceControlRevision.h"

class FPlasticSourceControlChangelistState;
class FPlasticSourceControlCommand;
class FPlasticSourceControlProvider;
class FPlasticSourceControlState;
struct FSoftwareVersion;
typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;
typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;
typedef TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStateRef;

enum class EWorkspaceState;

namespace PlasticSourceControlUtils
{

/**
 * Run a Plastic command - the result is the output of cm, as a multi-line string.
 *
 * @param	InCommand			The Plastic command - e.g. commit
 * @param	InParameters		The parameters to the Plastic command
 * @param	InFiles				The files to be operated on
 * @param	OutResults			The results (from StdOut) as a multi-line string.
 * @param	OutErrors			Any errors (from StdErr) as a multi-line string.
 * @returns true if the command succeeded and returned no errors
 */
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors);

/**
 * Run a Plastic command - the result is parsed in an array of strings.
 *
 * @param	InCommand			The Plastic command - e.g. commit
 * @param	InParameters		The parameters to the Plastic command
 * @param	InFiles				The files to be operated on
 * @param	OutResults			The results (from StdOut) as an array per-line
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 * @returns true if the command succeeded and returned no errors
 */
bool RunCommand(const FString& InCommand, const TArray<FString>& InParameters, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages);

/**
 * Find the path to the Plastic binary: for now relying on the Path to access the "cm" command.
 */
FString FindPlasticBinaryPath();

/**
 * Open the Desktop Application.
 *
 * @param	bInBranchExplorer	Should it open the Branch Explorer instead of the Workspace Explorer (false by default)?
 */
bool OpenDesktopApplication(const bool bInBranchExplorer = false);

/**
 * Open the Desktop Application for diffing a Changeset.
 *
 * @param	InChangesetId		Changeset to diff
 */
bool OpenDesktopApplicationForDiff(const int32 InChangesetId);

/**
 * Open the Desktop Application for diffing a couple of Changesets.
 *
 * @param	InChangesetIdSrc	Source (Left) Changeset to diff
 * @param	InChangesetIdDst	Destination (Right) Changeset to diff
 */
bool OpenDesktopApplicationForDiff(const int32 InChangesetIdSrc, const int32 InChangesetIdDst);

/**
 * Open the Desktop Application for diffing a whole Branch.
 *
 * @param	InBranchName		Name of the Branch to diff
 */
bool OpenDesktopApplicationForDiff(const FString& InBranchName);

/**
 * Open the Unity Cloud Dashboard on the page to show and manage Lock Rules.
 *
 * @param	InOrganizationName	Name of the organization to use, from the server URL
 */
void OpenLockRulesInCloudDashboard(const FString& InOrganizationName);

/**
 * Find the root of the Plastic workspace, looking from the GameDir and upward in its parent directories
 * @param InPathToGameDir		The path to the Game Directory
 * @param OutWorkspaceRoot		The path to the root directory of the Plastic workspace if found, else the path to the GameDir
 * @returns true if the command succeeded
 */
bool GetWorkspacePath(const FString& InPathToGameDir, FString& OutWorkspaceRoot);

/**
 * Get Unity Version Control CLI version
 * @param	OutCliVersion		Version of the Unity Version Control Command Line Interface tool
 * @returns true if the command succeeded
*/
bool GetPlasticScmVersion(FSoftwareVersion& OutPlasticScmVersion);

/**
 * Get Unity Version Control CLI location
 * @param	OutCmLocation		Path to the "cm" executable
 * @returns true if the command succeeded
*/
bool GetCmLocation(FString& OutCmLocation);

/**
 * Checks weather Unity Version Control is configured to set files as read-only on update & checkin
 * @returns true if SetFilesAsReadOnly is enabled in client.conf
*/
bool GetConfigSetFilesAsReadOnly();

/**
 * Get from config the name of the default cloud organization or the url of the default on-prem server
 * @returns Returns the location of the default repository server
*/
FString GetConfigDefaultRepServer();

/**
 * Get Unity Version Control user for the default server
 * @returns	Name of the Unity Version Control user configured globally
 */
FString GetDefaultUserName();

/**
 * Get Unity Version Control user for the specified server
 * @param	InServerUrl		Name of the specified server
 * @returns	Name of the Unity Version Control user for the specified server
 */
FString GetProfileUserName(const FString& InServerUrl);

/**
 * Get workspace name
 * @param	InWorkspaceRoot		The workspace from where to run the command - typically the Project path
 * @param	OutWorkspaceName	Name of the current workspace
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
*/
bool GetWorkspaceName(const FString& InWorkspaceRoot, FString& OutWorkspaceName, TArray<FString>& OutErrorMessages);

/**
 * Get workspace info: the current branch, repository name, and server URL
 *
 * @param	OutWorkspaceSelector	Name of the current branch, changeset or label depending on the workspace selector
 * @param	OutBranchName		Name of the current branch
 * @param	OutRepositoryName	Name of the repository of the current workspace
 * @param	OutServerUrl		URL/Port of the server of the repository
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 */
bool GetWorkspaceInfo(FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl, TArray<FString>& OutErrorMessages);


/**
 * Get the current changeset number from the workspace (-1 for a partial workspace)
 *
 * @param	OutChangesetNumber	Current changeset number
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 */
bool GetChangesetNumber(int32& OutChangesetNumber, TArray<FString>& OutErrorMessages);

/**
 * Get workspace info and check the connection to the server
 *
 * @param	OutWorkspaceSelector	Name of the current branch, changeset or label depending on the workspace selector
 * @param	OutBranchName			Name of the current branch when available
 * @param	OutRepositoryName		Name of the repository of the current workspace
 * @param	OutServerUrl			URL/Port of the server of the repository
 * @param	OutInfoMessages			Result of the connection test
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunCheckConnection(FString& OutWorkspaceSelector, FString& OutBranchName, FString& OutRepositoryName, FString& OutServerUrl, TArray<FString>& OutInfoMessages, TArray<FString>& OutErrorMessages);

/**
 * Use the Project Settings to replace Unity Version Control full username/e-mail by a shorter version for display.
 *
 * Used when retrieving the username of a revision, to display in history and content browser asset tooltip.
 *
 * @param	InUserName			The Unity Version Control username to shorten for display.
 */
FString UserNameToDisplayName(const FString& InUserName);

/**
 * Invalidate the cache of locks so that the next call to RunListLocks() will not use it and actually run the cm lock list command
 */
void InvalidateLocksCache();

/**
 * Run a Plastic "lock list" command and parse it.
 *
 * @param	InProvider				The source control provider to get the repository and current branch to ask the locks for
 * @param   bInForAllDestBranches	Retrieve locks for all destination branches, or restrict them to only those applying to the working branch
 * @param	OutLocks				The list of locks
 * @returns true if the command succeeded and returned no errors
 */
bool RunListLocks(const FPlasticSourceControlProvider& InProvider, const bool bInForAllDestBranches, TArray<FPlasticSourceControlLockRef>& OutLocks);

/**
 * Get locks applying to the working branch for the specified files.
 *
 * @param	InProvider			The source control provider to get the repository and current branch to ask the locks for
 * @param	InFiles				The files to be operated on (server paths)
 * @return	OutLocks			The list of corresponding locks if any
 */
TArray<FPlasticSourceControlLockRef> GetLocksForWorkingBranch(const FPlasticSourceControlProvider& InProvider, const TArray<FString>& InFiles);

/**
 * Get the list of filenames from the list of locks
 * @param	InWorkspaceRoot		The workspace from where to run the command - typically the Project path
 * @param	InLocks				Locks to get the file names for
 * @return	List of absolute filenames
*/
TArray<FString> LocksToFileNames(const FString InWorkspaceRoot, const TArray<FPlasticSourceControlLockRef>& InLocks);

// Specify the "search type" for the "status" command
enum class EStatusSearchType
{
	All,			// status --all --ignored (this can take much longer, searching for local changes, especially on the first call)
	ControlledOnly	// status --controlledchanged
};

/**
 * Run a Plastic "status" command and parse it.
 *
 * @param	InFiles				The files to be operated on
 * @param	InSearchType		Call "status" with "--all", or with just "--controlledchanged" when doing only a quick check following a source control operation
 * @param	bInUpdateHistory	If getting the history of files, force execute the fileinfo command required to do get RepSpec of xlinks (history view or visual diff)
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 * @param	OutStates			States of the files
 * @param	OutChangeset		The current Changeset Number
 * @returns true if the command succeeded and returned no errors
 */
bool RunUpdateStatus(const TArray<FString>& InFiles, const EStatusSearchType InSearchType, const bool bInUpdateHistory, TArray<FString>& OutErrorMessages, TArray<FPlasticSourceControlState>& OutStates, int32& OutChangeset);

/**
 * Run a Plastic "cat" command to dump the binary content of a revision into a file.
 *
 * @param	InRevSpec				The revision specification to get
 * @param	InDumpFileName			The temporary file to dump the revision
 * @returns true if the command succeeded and returned no errors
*/
bool RunGetFile(const FString& InRevSpec, const FString& InDumpFileName);

/**
 * Run Plastic "history" and "log" commands and parse their XML results.
 *
 * @param	bInUpdateHistory	If getting the history of files, versus only checking the heads of branches to detect newer commits
 * @param	InOutStates			The file states to update with the history
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 */
bool RunGetHistory(const bool bInUpdateHistory, TArray<FPlasticSourceControlState>& InOutStates, TArray<FString>& OutErrorMessages);

/**
 * Run a Plastic "update" command to sync the workspace and parse its XML results.
 *
 * @param	InFiles					The files or paths to sync
 * @param	bInIsPartialWorkspace	Whether running on a partial/gluon or regular/full workspace
 * @param	InChangesetId			The optional changeset to sync to (leave empty to sync to the latest in the branch)
 * @param	OutUpdatedFiles			The files that where updated
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunUpdate(const TArray<FString>& InFiles, const bool bInIsPartialWorkspace, const FString& InChangesetId, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages);

/**
 * Run a Plastic "status --changelist --xml" and parse its XML result.
 * @param	OutChangelistsStates	The list of changelists (without their files)
 * @param	OutCLFilesStates		The list of files per changelist
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetChangelists(TArray<FPlasticSourceControlChangelistState>& OutChangelistsStates, TArray<TArray<FPlasticSourceControlState>>& OutCLFilesStates, TArray<FString>& OutErrorMessages);

/**
 * Run find "shelves where owner='me'" and for each shelve matching a changelist a "diff sh:<ShelveId>" and parse their results.
 * @param	InOutChangelistsStates	The list of changelists, filled with their shelved files
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetShelves(TArray<FPlasticSourceControlChangelistState>& InOutChangelistsStates, TArray<FString>& OutErrorMessages);

/**
 * Run find "shelves where ShelveId='NNN'" and a "diff sh:<ShelveId>" and parse their results.
 * @param	InShelveId			Shelve Id
 * @param	OutComment			Shelve Comment
 * @param	OutDate				Shelve Date
 * @param	OutOwner			Shelve Owner
 * @param	OutStates			Files in the shelve and their base revision id
 * @param	OutErrorMessages	Any errors (from StdErr) as an array per-line
 */
bool RunGetShelve(const int32 InShelveId, FString& OutComment, FDateTime& OutDate, FString& OutOwner, TArray<FPlasticSourceControlRevision>& OutBaseRevisions, TArray<FString>& OutErrorMessages);

/**
 * Add a file to the shelve associated with a changelist.
 * @param	InOutChangelistsState	The changelist to add the file to
 * @param	InFilename				The file to add to the shelve
 * @param	InShelveStatus			The status of the file
 * @param	InMovedFrom				If moved, the original filename
*/
void AddShelvedFileToChangelist(FPlasticSourceControlChangelistState& InOutChangelistsState, FString&& InFilename, EWorkspaceState InShelveStatus, FString&& InMovedFrom);

/**
 * Run find "changesets where date >= 'YYYY-MM-DD'" and parse the results.
 * @param	InFromDate				The date to search from
 * @param	OutChangesets			The list of changesets, without their files
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 *
 * @see RunGetChangesetFiles() below used to populated a specific changeset with its list of files
 */
bool RunGetChangesets(const FDateTime& InFromDate, TArray<FPlasticSourceControlChangesetRef>& OutChangesets, TArray<FString>& OutErrorMessages);

/**
 * Run "log cs:<ChangesetId> --xml" and parse the results to populate the files from the specified changeset.
 * @param	InChangeset				The changeset to get the files changed
 * @param	OutFiles				The files changed in the specified changeset
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetChangesetFiles(const FPlasticSourceControlChangesetRef& InChangeset, TArray<FPlasticSourceControlStateRef>& OutFiles, TArray<FString>& OutErrorMessages);

/**
 * Run find "branches where date >= 'YYYY-MM-DD' or changesets >= 'YYYY-MM-DD'" and parse the results.
 * @param	InFromDate				The date to search from
 * @param	OutBranches				The list of branches
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunGetBranches(const FDateTime& InFromDate, TArray<FPlasticSourceControlBranchRef>& OutBranches, TArray<FString>& OutErrorMessages);

/**
 * Run switch br:/name and parse the results.
 * @param	InBranchName			The name of the branch to switch the workspace to (optional, only used if no InChangesetId)
 * @param	InChangesetId			The name of the changeset to switch the workspace to (optional, overrides InBranchName if set)
 * @param	bInIsPartialWorkspace	Whether running on a partial/gluon or regular/full workspace
 * @param	OutUpdatedFiles			The files that where updated
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunSwitch(const FString& InBranchName, const int32 InChangesetId, const bool bInIsPartialWorkspace, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages);

/**
 * Run merge br:/name and parse the results.
 * @param	InBranchName			The name of the branch to merge to the current branch
 * @param	OutUpdatedFiles			The files that where updated
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunMergeBranch(const FString& InBranchName, TArray<FString>& OutUpdatedFiles, TArray<FString>& OutErrorMessages);

/**
 * Run branch create <name> --commentsfile
 * @param	InBranchName			The name of the branch to create
 * @param	InComment				The comment for the new branch to create
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunCreateBranch(const FString& InBranchName, const FString& InComment, TArray<FString>& OutErrorMessages);

/**
 * Run branch rename <old name> <new name>
 * @param	InOldName				The old name of the branch to rename
 * @param	InNewName				The new name to rename the branch to
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunRenameBranch(const FString& InOldName, const FString& InNewName, TArray<FString>& OutErrorMessages);

/**
 * Run branch delete <name1> <name2 ...>
 * @param	InBranchNames			The name of the branch(es) to delete
 * @param	OutErrorMessages		Any errors (from StdErr) as an array per-line
 */
bool RunDeleteBranches(const TArray<FString>& InBranchNames, TArray<FString>& OutErrorMessages);

/**
 * Helper function for various commands to update cached states.
 * @returns true if any states were updated
 */
bool UpdateCachedStates(TArray<FPlasticSourceControlState>&& InStates);

/**
 * Remove redundant errors (that contain a particular string) and also
 * update the commands success status if all errors were removed.
 */
void RemoveRedundantErrors(FPlasticSourceControlCommand& InCommand, const FString& InFilter);

/**
 * Change LogSourceControl verbosity level at startup and when toggled from the Plastic Source Control Settings
 *
 * Override to Verbose or back to Log, but only if the current log verbosity is not already set to VeryVerbose
 */
void SwitchVerboseLogs(const bool bInEnable);

/**
 * Find the best(longest) common directory between two paths, terminated by a slash, returning an empty string if none.
 * Assumes that both input strings are already normalized paths, slash delimited, for performance reason.
 */
FString FindCommonDirectory(const FString& InPath1, const FString& InPath2);

} // namespace PlasticSourceControlUtils
