// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SoftwareVersion.h"

/**
 * List of key versions of Unity Version Control enabling new features for "cm" CLI.
 *
 * Unity Version Control version strings in the form "X.Y.Z.C", ie Major.Minor.Patch.Build (as returned by GetPlasticScmVersion)
*/
namespace PlasticSourceControlVersions
{
	// Oldest supported version of "cm"
	// 9.0.16.4839 cm changelist 'persistent' flag now contain a '--' prefix.
	// https://plasticscm.com/download/releasenotes/9.0.16.4839 (2021/01/05)
	static FSoftwareVersion OldestSupported(TEXT("9.0.16.4839"));

	// 11.0.16.7248 add support for --descriptionfile for multi-line descriptions and support for special characters
	// https://plasticscm.com/download/releasenotes/11.0.16.7248 (2022/07/28)
	static const FSoftwareVersion NewChangelistFileArgs(TEXT("11.0.16.7248"));

	// 11.0.16.7504 add support for 'cm shelveset apply' to only unshelved the desired changes from a shelve
	// https://plasticscm.com/download/releasenotes/11.0.16.7504 (2022/10/13)
	static const FSoftwareVersion ShelvesetApplySelection(TEXT("11.0.16.7504"));

	// 11.0.16.7608 add support for history --limit. It displays the N last revisions of the specified items.
	// https://plasticscm.com/download/releasenotes/11.0.16.7608 (2022/11/07)
	// NOTE: this is also the first version aware of Unreal Engine 5 "UnrealEditor.exe"
	static const FSoftwareVersion NewHistoryLimit(TEXT("11.0.16.7608"));

	// 11.0.16.7665 add support for undocheckout --keepchanges. It allows undo checkout and preserve all local changes.
	// https://plasticscm.com/download/releasenotes/11.0.16.7665 (2022/12/01)
	static const FSoftwareVersion UndoCheckoutKeepChanges(TEXT("11.0.16.7665"));

	// 11.0.16.7709 add support for status --iscochanged. It uses "CO+CH" vs "CO" to distinguish CheckedOut with/without changes.
	// https://plasticscm.com/download/releasenotes/11.0.16.7709 (2023/01/12)
	static const FSoftwareVersion StatusIsCheckedOutChanged(TEXT("11.0.16.7709"));

	// 11.0.16.7726 add support for merge --xml
	// https://plasticscm.com/download/releasenotes/11.0.16.7726 (2023/01/19)
	static const FSoftwareVersion MergeXml(TEXT("11.0.16.7726"));

	// 11.0.16.8101 Introducing Smart Locks
	// https://plasticscm.com/download/releasenotes/11.0.16.8101 (2023/07/06)
	// 11.0.16.8133 add support for non-localized --datetimeformat
	// https://plasticscm.com/download/releasenotes/11.0.16.8133 (2023/08/03)
	static const FSoftwareVersion SmartLocks(TEXT("11.0.16.8133"));

	// 11.0.16.8445 lock list new --workingbranch option to only list the locks that apply to the destination branch of the specified branch.
	// https://plasticscm.com/download/releasenotes/11.0.16.8445 (2024/02/22)
	static const FSoftwareVersion WorkingBranch(TEXT("11.0.16.8445"));

} // namespace PlasticSourceControlVersions
