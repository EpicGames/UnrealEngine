// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlasticSourceControlWorkspaceCreation.h"

#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlModule.h"

#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "Misc/Paths.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FPlasticSourceControlWorkspaceCreation"


void FPlasticSourceControlWorkspaceCreation::MakeWorkspace(const FParameters& InParameters)
{
	WorkspaceParams = InParameters;

	// 1.a. Create a repository (if not already existing) and a workspace: launch an asynchronous MakeWorkspace operation
	LaunchMakeWorkspaceOperation();
}

/// 1. Create a repository (if not already existing) and a workspace
void FPlasticSourceControlWorkspaceCreation::LaunchMakeWorkspaceOperation()
{
	TSharedRef<FPlasticMakeWorkspace, ESPMode::ThreadSafe> MakeWorkspaceOperation = ISourceControlOperation::Create<FPlasticMakeWorkspace>();
	MakeWorkspaceOperation->WorkspaceName = WorkspaceParams.WorkspaceName.ToString();
	MakeWorkspaceOperation->RepositoryName = WorkspaceParams.RepositoryName.ToString();
	MakeWorkspaceOperation->ServerUrl = WorkspaceParams.ServerUrl.ToString();
	MakeWorkspaceOperation->bPartialWorkspace = WorkspaceParams.bCreatePartialWorkspace;

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	ECommandResult::Type Result = Provider.Execute(MakeWorkspaceOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlWorkspaceCreation::OnMakeWorkspaceOperationComplete));
	if (Result == ECommandResult::Succeeded)
	{
		Notification.DisplayInProgress(MakeWorkspaceOperation->GetInProgressString());
	}
	else
	{
		FNotification::DisplayFailure(MakeWorkspaceOperation.Get());
	}
}

void FPlasticSourceControlWorkspaceCreation::OnMakeWorkspaceOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	OnSourceControlOperationComplete(InOperation, InResult);

	if (InResult == ECommandResult::Succeeded)
	{
		// Launch the next asynchronous operation
		LaunchMarkForAddOperation();
	}
}

/// 2. Add all project files to Source Control (.uproject, Config/, Content/, Source/ files and ignore.conf if any)
void FPlasticSourceControlWorkspaceCreation::LaunchMarkForAddOperation()
{
	TSharedRef<FMarkForAdd, ESPMode::ThreadSafe> MarkForAddOperation = ISourceControlOperation::Create<FMarkForAdd>();
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();

	// 1.b. Check the new workspace status to enable connection
	Provider.CheckPlasticAvailability();

	if (Provider.IsWorkspaceFound())
	{
		// 2. Add all project files to Source Control (.uproject, Config/, Content/, Source/ files and ignore.conf if any)
		const TArray<FString> ProjectFiles = GetProjectFiles();
		ECommandResult::Type Result = Provider.Execute(MarkForAddOperation, ProjectFiles, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlWorkspaceCreation::OnMarkForAddOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			Notification.DisplayInProgress(MarkForAddOperation->GetInProgressString());
		}
		else
		{
			FNotification::DisplayFailure(MarkForAddOperation.Get());
		}
	}
	else
	{
		FNotification::DisplayFailure(MarkForAddOperation.Get());
	}
}

void FPlasticSourceControlWorkspaceCreation::OnMarkForAddOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	OnSourceControlOperationComplete(InOperation, InResult);

	if (InResult == ECommandResult::Succeeded)
	{
		// Launch the next asynchronous operation
		LaunchCheckInOperation();
	}
}

/// 3. Launch an asynchronous "CheckIn" operation and start another ongoing notification
void FPlasticSourceControlWorkspaceCreation::LaunchCheckInOperation()
{
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
	CheckInOperation->SetDescription(WorkspaceParams.InitialCommitMessage);
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const TArray<FString> ProjectFiles = GetProjectFiles(); // Note: listing files and folders is only needed for the update status operation following the checkin to know on what to operate
	ECommandResult::Type Result = Provider.Execute(CheckInOperation, ProjectFiles, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw(this, &FPlasticSourceControlWorkspaceCreation::OnCheckInOperationComplete));
	if (Result == ECommandResult::Succeeded)
	{
		Notification.DisplayInProgress(CheckInOperation->GetInProgressString());
	}
	else
	{
		FNotification::DisplayFailure(CheckInOperation.Get());
	}
}

void FPlasticSourceControlWorkspaceCreation::OnCheckInOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	OnSourceControlOperationComplete(InOperation, InResult);

	// Note: no more operation to launch, the workspace is ready to use
}

void FPlasticSourceControlWorkspaceCreation::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	Notification.RemoveInProgress();

	// Report result with a notification
	FNotification::DisplayResult(InOperation, InResult);
}

/** Path to the "ignore.conf" file */
const FString FPlasticSourceControlWorkspaceCreation::GetIgnoreFileName() const
{
	const FString PathToWorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();
	const FString IgnoreFileName = FPaths::Combine(*PathToWorkspaceRoot, TEXT("ignore.conf"));
	return IgnoreFileName;
}

/** List of files to add to Source Control (.uproject, Config/, Content/, Source/ files and ignore.conf if any) */
TArray<FString> FPlasticSourceControlWorkspaceCreation::GetProjectFiles() const
{
	TArray<FString> ProjectFiles;
	ProjectFiles.Add(FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()));
	ProjectFiles.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectConfigDir()));
	ProjectFiles.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
	if (FPaths::DirectoryExists(FPaths::GameSourceDir()))
	{
		ProjectFiles.Add(FPaths::ConvertRelativePathToFull(FPaths::GameSourceDir()));
	}
	if (FPaths::FileExists(GetIgnoreFileName()))
	{
		ProjectFiles.Add(GetIgnoreFileName());
	}
	return ProjectFiles;
}

#undef LOCTEXT_NAMESPACE
