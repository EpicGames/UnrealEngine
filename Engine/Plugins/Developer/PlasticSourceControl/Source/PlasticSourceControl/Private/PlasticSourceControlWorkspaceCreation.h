// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Notification.h"

#include "ISourceControlProvider.h"
#include "ISourceControlOperation.h"

class FPlasticSourceControlWorkspaceCreation
{
public:
	struct FParameters
	{
		FText WorkspaceName;
		FText RepositoryName;
		FText ServerUrl;
		bool bCreatePartialWorkspace = false;
		bool bAutoInitialCommit = true;
		FText InitialCommitMessage;
	};

	FParameters WorkspaceParams;

	void MakeWorkspace(const FParameters& InParameters);

private:
	/** Launch initial asynchronous add and commit operations */
	void LaunchMakeWorkspaceOperation();
	void LaunchMarkForAddOperation();
	void LaunchCheckInOperation();

	/** Delegates called when a source control operation has completed */
	void OnMakeWorkspaceOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnMarkForAddOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnCheckInOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	/** Generic notification handler */
	void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Ongoing notification for a long-running asynchronous source control operation, if any */
	FNotification Notification;

	const FString GetIgnoreFileName() const;
	TArray<FString> GetProjectFiles() const;
};
