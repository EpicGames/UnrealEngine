// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlBranchRow.h"

#include "PlasticSourceControlBranch.h"
#include "PlasticSourceControlUtils.h"

#include "Widgets/Text/STextBlock.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

FName PlasticSourceControlBranchesListViewColumn::Name::Id() { return TEXT("Name"); }
FText PlasticSourceControlBranchesListViewColumn::Name::GetDisplayText() { return LOCTEXT("Name_Column", "Name"); }
FText PlasticSourceControlBranchesListViewColumn::Name::GetToolTipText() { return LOCTEXT("Name_Column_Tooltip", "Displays the asset/file name"); }

FName PlasticSourceControlBranchesListViewColumn::Repository::Id() { return TEXT("Repository"); }
FText PlasticSourceControlBranchesListViewColumn::Repository::GetDisplayText() { return LOCTEXT("Repository_Column", "Repository"); }
FText PlasticSourceControlBranchesListViewColumn::Repository::GetToolTipText() { return LOCTEXT("Repository_Column_Tooltip", "Displays the repository where the branch has been created"); }

FName PlasticSourceControlBranchesListViewColumn::CreatedBy::Id() { return TEXT("CreatedBy"); }
FText PlasticSourceControlBranchesListViewColumn::CreatedBy::GetDisplayText() { return LOCTEXT("CreatedBy_Column", "Created by"); }
FText PlasticSourceControlBranchesListViewColumn::CreatedBy::GetToolTipText() { return LOCTEXT("CreatedBy_Column_Tooltip", "Displays the name of the creator of the branch"); }

FName PlasticSourceControlBranchesListViewColumn::Date::Id() { return TEXT("Date"); }
FText PlasticSourceControlBranchesListViewColumn::Date::GetDisplayText() { return LOCTEXT("Date_Column", "Creation date"); }
FText PlasticSourceControlBranchesListViewColumn::Date::GetToolTipText() { return LOCTEXT("Date_Column_Tooltip", "Displays the branch creation date"); }

FName PlasticSourceControlBranchesListViewColumn::Comment::Id() { return TEXT("Comment"); }
FText PlasticSourceControlBranchesListViewColumn::Comment::GetDisplayText() { return LOCTEXT("Comment_Column", "Comment"); }
FText PlasticSourceControlBranchesListViewColumn::Comment::GetToolTipText() { return LOCTEXT("Comment_Column_Tooltip", "Displays the branch comment"); }

void SPlasticSourceControlBranchRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	BranchToVisualize = InArgs._BranchToVisualize.Get();
	bIsCurrentBranch = InArgs._bIsCurrentBranch;
	HighlightText = InArgs._HighlightText;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> SPlasticSourceControlBranchRow::GenerateWidgetForColumn(const FName& InColumnId)
{
	const FSlateFontInfo FontInfo = bIsCurrentBranch ? FAppStyle::GetFontStyle("BoldFont") : FAppStyle::GetFontStyle("NormalFont");

	if (InColumnId == PlasticSourceControlBranchesListViewColumn::Name::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(BranchToVisualize->Name))
			.ToolTipText(FText::FromString(BranchToVisualize->Name))
			.Margin(FMargin(6.f, 1.f))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlBranchesListViewColumn::Repository::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(BranchToVisualize->Repository))
			.ToolTipText(FText::FromString(BranchToVisualize->Repository))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(PlasticSourceControlUtils::UserNameToDisplayName(BranchToVisualize->CreatedBy)))
			.ToolTipText(FText::FromString(BranchToVisualize->CreatedBy))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlBranchesListViewColumn::Date::Id())
	{
		return SNew(STextBlock)
			.Text(FText::AsDateTime(BranchToVisualize->Date))
			.ToolTipText(FText::AsDateTime(BranchToVisualize->Date))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo);
	}
	else if (InColumnId == PlasticSourceControlBranchesListViewColumn::Comment::Id())
	{
		// Make each comment fit on a single line to preserve the table layout
		FString CommentOnOneLine = BranchToVisualize->Comment;
		CommentOnOneLine.ReplaceCharInline(TEXT('\n'), TEXT(' '), ESearchCase::CaseSensitive);

		return SNew(STextBlock)
			.Text(FText::FromString(MoveTemp(CommentOnOneLine)))
			.ToolTipText(FText::FromString(BranchToVisualize->Comment))
			.Margin(FMargin(6.f, 1.f))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
