// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;
typedef TSharedPtr<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchPtr;

class FPlasticSourceControlBranch;

/** Lists the unique columns used in the list view displaying branches. */
namespace PlasticSourceControlBranchesListViewColumn
{
	/** The branch Name column. */
	namespace Name // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The branch Repository column. */
	namespace Repository // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The branch CreatedBy column. */
	namespace CreatedBy // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The branch Date column. */
	namespace Date // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The branch Comment column. */
	namespace Comment // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}
} // namespace PlasticSourceControlBranchesListViewColumn

class SPlasticSourceControlBranchRow : public SMultiColumnTableRow<FPlasticSourceControlBranchRef>
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlBranchRow)
		: _BranchToVisualize(nullptr)
		, _bIsCurrentBranch(false)
		, _HighlightText()
	{
	}
		SLATE_ARGUMENT(FPlasticSourceControlBranchPtr, BranchToVisualize)
		SLATE_ARGUMENT(bool, bIsCurrentBranch)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:
	/**
	* Construct a row child widgets of the ListView.
	*
	* @param InArgs Parameters including the branch to visualize in this row.
	* @param InOwner The owning ListView.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

private:
	/** The branch that we are visualizing in this row. */
	FPlasticSourceControlBranch* BranchToVisualize;

	/** True if this is the current branch, to be highlighted on the list of branches. */
	bool bIsCurrentBranch;

	/** The search text to highlight if any */
	TAttribute<FText> HighlightText;
};
