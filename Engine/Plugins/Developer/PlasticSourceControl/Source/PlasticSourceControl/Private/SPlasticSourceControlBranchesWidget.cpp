// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlBranchesWidget.h"

#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlProjectSettings.h"
#include "PlasticSourceControlBranch.h"
#include "PlasticSourceControlUtils.h"
#include "PlasticSourceControlVersions.h"
#include "SPlasticSourceControlBranchRow.h"
#include "SPlasticSourceControlCreateBranch.h"
#include "SPlasticSourceControlDeleteBranches.h"
#include "SPlasticSourceControlRenameBranch.h"

#include "PackageUtils.h"

#include "ISourceControlModule.h"

#include "Logging/MessageLog.h"
#include "ToolMenus.h"
#include "ToolMenuContext.h"

#include "Misc/ComparisonUtility.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

void SPlasticSourceControlBranchesWidget::Construct(const FArguments& InArgs)
{
	ISourceControlModule::Get().RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnSourceControlProviderChanged));
	// register for any source control change to detect any change of branch from the Changesets window
	SourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SPlasticSourceControlBranchesWidget::HandleSourceControlStateChanged));

	WorkspaceSelector = FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector();

	SearchTextFilter = MakeShared<TTextFilter<const FPlasticSourceControlBranch&>>(TTextFilter<const FPlasticSourceControlBranch&>::FItemToStringArray::CreateSP(this, &SPlasticSourceControlBranchesWidget::PopulateItemSearchStrings));
	SearchTextFilter->OnChanged().AddSP(this, &SPlasticSourceControlBranchesWidget::OnRefreshUI);

	FromDateInDaysValues.Add(TPair<int32, FText>(7, FText::FromString(TEXT("Last week"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(15, FText::FromString(TEXT("Last 15 days"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(30, FText::FromString(TEXT("Last month"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(91, FText::FromString(TEXT("Last 3 months"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(182, FText::FromString(TEXT("Last 6 months"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(365, FText::FromString(TEXT("Last year"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(-1, FText::FromString(TEXT("All time"))));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot() // For the toolbar (Search box and Refresh button)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						CreateToolBar()
					]
					+SHorizontalBox::Slot()
					.MaxWidth(10.0f)
					[
						SNew(SSpacer)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(300.0f)
					[
						SAssignNew(BranchSearchBox, SSearchBox)
						.HintText(LOCTEXT("SearchBranches", "Search Branches"))
						.ToolTipText(LOCTEXT("PlasticBranchesSearch_Tooltip", "Filter the list of branches by keyword."))
						.OnTextChanged(this, &SPlasticSourceControlBranchesWidget::OnSearchTextChanged)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(125.0f)
					.Padding(10.f, 0.f)
					[
						SNew(SComboButton)
						.ToolTipText(LOCTEXT("PlasticBranchesDate_Tooltip", "Filter the list of branches by date of activity."))
						.OnGetMenuContent(this, &SPlasticSourceControlBranchesWidget::BuildFromDateDropDownMenu)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text_Lambda([this]() { return FromDateInDaysValues[FromDateInDays]; })
						]
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					// Button to open the Changesets View
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 0.0f))
					.ToolTipText(LOCTEXT("PlasticChangesetsWindowTooltip", "Open the Changesets window."))
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([]()
						{
							FPlasticSourceControlModule::Get().GetChangesetsWindow().OpenTab();
							return FReply::Handled();
						})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SourceControl.Actions.History"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
							.Text(LOCTEXT("PlasticChangesetsWindow", "Changesets"))
						]
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					// Button to open the Branch Explorer
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 0.0f))
					.ToolTipText(LOCTEXT("PlasticBranchExplorerTooltip", "Open the Branch Explorer of the Desktop Application for the current workspace."))
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([]()
						{
							PlasticSourceControlUtils::OpenDesktopApplication(true);
							return FReply::Handled();
						})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SourceControl.Branch"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
							.Text(LOCTEXT("OpenBranchExplorer", "Branch Explorer"))
						]
					]
				]
			]
		]
		+SVerticalBox::Slot() // The main content: the list of branches
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				CreateContentPanel()
			]
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.FillHeight(1.0f)
			[
				// Text to display when there is no branch displayed
				SNew(STextBlock)
				.Text(LOCTEXT("NoBranch", "There is no branch to display."))
				.Visibility_Lambda([this]() { return SourceControlBranches.Num() ? EVisibility::Collapsed : EVisibility::Visible; })
			]
		]
		+SVerticalBox::Slot() // Status bar (Always visible)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(FMargin(0.f, 3.f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return RefreshStatus; })
					.Margin(FMargin(5.f, 0.f))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::FromString(WorkspaceSelector); })
					.ToolTipText(LOCTEXT("PlasticBranchCurrent_Tooltip", "Current branch."))
				]
			]
		]
	];
}

TSharedRef<SWidget> SPlasticSourceControlBranchesWidget::CreateToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]() { bShouldRefresh = true; })),
		NAME_None,
		LOCTEXT("SourceControl_RefreshButton", "Refresh"),
		LOCTEXT("SourceControl_RefreshButton_Tooltip", "Refreshes branches from revision control provider."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"));

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SPlasticSourceControlBranchesWidget::CreateContentPanel()
{
	// Inspired by Engine\Source\Editor\SourceControlWindows\Private\SSourceControlChangelists.cpp
	// TSharedRef<SListView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateChangelistFilesView()

	UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
	if (!Settings->bShowBranchRepositoryColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlBranchesListViewColumn::Repository::Id());
	}
	if (!Settings->bShowBranchCreatedByColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlBranchesListViewColumn::CreatedBy::Id());
	}
	if (!Settings->bShowBranchDateColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlBranchesListViewColumn::Date::Id());
	}
	if (!Settings->bShowBranchCommentColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlBranchesListViewColumn::Comment::Id());
	}

	TSharedRef<SListView<FPlasticSourceControlBranchRef>> BranchView = SNew(SListView<FPlasticSourceControlBranchRef>)
		.ListItemsSource(&BranchRows)
		.OnGenerateRow(this, &SPlasticSourceControlBranchesWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SPlasticSourceControlBranchesWidget::OnOpenContextMenu)
		.OnMouseButtonDoubleClick(this, &SPlasticSourceControlBranchesWidget::OnItemDoubleClicked)
		.OnItemToString_Debug_Lambda([this](FPlasticSourceControlBranchRef Branch) { return Branch->Name; })
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList(HiddenColumnsList)
			.OnHiddenColumnsListChanged(this, &SPlasticSourceControlBranchesWidget::OnHiddenColumnsListChanged)

			+SHeaderRow::Column(PlasticSourceControlBranchesListViewColumn::Name::Id())
			.DefaultLabel(PlasticSourceControlBranchesListViewColumn::Name::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlBranchesListViewColumn::Name::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlBranchesWidget::GetColumnSortPriority, PlasticSourceControlBranchesListViewColumn::Name::Id())
			.SortMode(this, &SPlasticSourceControlBranchesWidget::GetColumnSortMode, PlasticSourceControlBranchesListViewColumn::Name::Id())
			.OnSort(this, &SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlBranchesListViewColumn::Repository::Id())
			.DefaultLabel(PlasticSourceControlBranchesListViewColumn::Repository::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlBranchesListViewColumn::Repository::GetToolTipText())
			.FillWidth(1.5f)
			.SortPriority(this, &SPlasticSourceControlBranchesWidget::GetColumnSortPriority, PlasticSourceControlBranchesListViewColumn::Repository::Id())
			.SortMode(this, &SPlasticSourceControlBranchesWidget::GetColumnSortMode, PlasticSourceControlBranchesListViewColumn::Repository::Id())
			.OnSort(this, &SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
			.DefaultLabel(PlasticSourceControlBranchesListViewColumn::CreatedBy::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlBranchesListViewColumn::CreatedBy::GetToolTipText())
			.FillWidth(2.5f)
			.SortPriority(this, &SPlasticSourceControlBranchesWidget::GetColumnSortPriority, PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
			.SortMode(this, &SPlasticSourceControlBranchesWidget::GetColumnSortMode, PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
			.OnSort(this, &SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlBranchesListViewColumn::Date::Id())
			.DefaultLabel(PlasticSourceControlBranchesListViewColumn::Date::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlBranchesListViewColumn::Date::GetToolTipText())
			.FillWidth(1.5f)
			.SortPriority(this, &SPlasticSourceControlBranchesWidget::GetColumnSortPriority, PlasticSourceControlBranchesListViewColumn::Date::Id())
			.SortMode(this, &SPlasticSourceControlBranchesWidget::GetColumnSortMode, PlasticSourceControlBranchesListViewColumn::Date::Id())
			.OnSort(this, &SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlBranchesListViewColumn::Comment::Id())
			.DefaultLabel(PlasticSourceControlBranchesListViewColumn::Comment::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlBranchesListViewColumn::Comment::GetToolTipText())
			.FillWidth(5.0f)
			.SortPriority(this, &SPlasticSourceControlBranchesWidget::GetColumnSortPriority, PlasticSourceControlBranchesListViewColumn::Comment::Id())
			.SortMode(this, &SPlasticSourceControlBranchesWidget::GetColumnSortMode, PlasticSourceControlBranchesListViewColumn::Comment::Id())
			.OnSort(this, &SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged)
		);

	BranchesListView = BranchView;

	return BranchView;
}

TSharedRef<ITableRow> SPlasticSourceControlBranchesWidget::OnGenerateRow(FPlasticSourceControlBranchRef InBranch, const TSharedRef<STableViewBase>& OwnerTable)
{
	const bool bIsCurrentBranch = InBranch->Name == WorkspaceSelector;
	return SNew(SPlasticSourceControlBranchRow, OwnerTable)
		.BranchToVisualize(InBranch)
		.bIsCurrentBranch(bIsCurrentBranch)
		.HighlightText_Lambda([this]() { return BranchSearchBox->GetText(); });
}

void SPlasticSourceControlBranchesWidget::OnHiddenColumnsListChanged()
{
	// Update and save config to reload it on the next Editor sessions
	if (BranchesListView && BranchesListView->GetHeaderRow())
	{
		UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
		Settings->bShowBranchRepositoryColumn = true;
		Settings->bShowBranchCreatedByColumn = true;
		Settings->bShowBranchDateColumn = true;
		Settings->bShowBranchCommentColumn = true;

		for (const FName& ColumnId : BranchesListView->GetHeaderRow()->GetHiddenColumnIds())
		{
			if (ColumnId == PlasticSourceControlBranchesListViewColumn::Repository::Id())
			{
				Settings->bShowBranchRepositoryColumn = false;
			}
			else if (ColumnId == PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
			{
				Settings->bShowBranchCreatedByColumn = false;
			}
			else if (ColumnId == PlasticSourceControlBranchesListViewColumn::Date::Id())
			{
				Settings->bShowBranchDateColumn = false;
			}
			else if (ColumnId == PlasticSourceControlBranchesListViewColumn::Comment::Id())
			{
				Settings->bShowBranchCommentColumn = false;
			}
		}
		Settings->SaveConfig();
	}
}

void SPlasticSourceControlBranchesWidget::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	BranchSearchBox->SetError(SearchTextFilter->GetFilterErrorText());
}

void SPlasticSourceControlBranchesWidget::PopulateItemSearchStrings(const FPlasticSourceControlBranch& InItem, TArray<FString>& OutStrings)
{
	InItem.PopulateSearchString(OutStrings);
}

void SPlasticSourceControlBranchesWidget::OnFromDateChanged(int32 InFromDateInDays)
{
	FromDateInDays = InFromDateInDays;
	bShouldRefresh = true;
}

TSharedRef<SWidget> SPlasticSourceControlBranchesWidget::BuildFromDateDropDownMenu()
{
	FMenuBuilder MenuBuilder(true, NULL);

	for (const auto & FromDateInDaysValue : FromDateInDaysValues)
	{
		FUIAction MenuAction(FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnFromDateChanged, FromDateInDaysValue.Key));
		MenuBuilder.AddMenuEntry(FromDateInDaysValue.Value, FromDateInDaysValue.Value, FSlateIcon(), MenuAction);
	}

	return MenuBuilder.MakeWidget();
}

void SPlasticSourceControlBranchesWidget::OnRefreshUI()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::OnRefreshUI);

	const int32 ItemCount = SourceControlBranches.Num();
	BranchRows.Empty(ItemCount);
	for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
	{
		const FPlasticSourceControlBranchRef& Item = SourceControlBranches[ItemIndex];
		if (SearchTextFilter->PassesFilter(Item.Get()))
		{
			BranchRows.Emplace(Item);
		}
	}

	if (GetListView())
	{
		SortBranchView();
		GetListView()->RequestListRefresh();
	}
}

EColumnSortPriority::Type SPlasticSourceControlBranchesWidget::GetColumnSortPriority(const FName InColumnId) const
{
	if (InColumnId == PrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (InColumnId == SecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

EColumnSortMode::Type SPlasticSourceControlBranchesWidget::GetColumnSortMode(const FName InColumnId) const
{
	if (InColumnId == PrimarySortedColumn)
	{
		return PrimarySortMode;
	}
	else if (InColumnId == SecondarySortedColumn)
	{
		return SecondarySortMode;
	}

	return EColumnSortMode::None;
}

void SPlasticSourceControlBranchesWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	if (InSortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedColumn = InColumnId;
		PrimarySortMode = InSortMode;

		if (InColumnId == SecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			SecondarySortedColumn = FName();
			SecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (InSortPriority == EColumnSortPriority::Secondary)
	{
		SecondarySortedColumn = InColumnId;
		SecondarySortMode = InSortMode;
	}

	if (GetListView())
	{
		SortBranchView();
		GetListView()->RequestListRefresh();
	}
}

void SPlasticSourceControlBranchesWidget::SortBranchView()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::SortBranchView);

	if (PrimarySortedColumn.IsNone() || BranchRows.Num() == 0)
	{
		return; // No column selected for sorting or nothing to sort.
	}

	auto CompareNames = [](const FPlasticSourceControlBranch* Lhs, const FPlasticSourceControlBranch* Rhs)
	{
		return UE::ComparisonUtility::CompareNaturalOrder(*Lhs->Name, *Rhs->Name);
	};

	auto CompareRepository = [](const FPlasticSourceControlBranch* Lhs, const FPlasticSourceControlBranch* Rhs)
	{
		return UE::ComparisonUtility::CompareNaturalOrder(*Lhs->Repository, *Rhs->Repository);
	};

	auto CompareCreatedBy = [](const FPlasticSourceControlBranch* Lhs, const FPlasticSourceControlBranch* Rhs)
	{
		return FCString::Stricmp(*Lhs->CreatedBy, *Rhs->CreatedBy);
	};

	auto CompareDate = [](const FPlasticSourceControlBranch* Lhs, const FPlasticSourceControlBranch* Rhs)
	{
		return Lhs->Date < Rhs->Date ? -1 : (Lhs->Date == Rhs->Date ? 0 : 1);
	};

	auto CompareComment = [](const FPlasticSourceControlBranch* Lhs, const FPlasticSourceControlBranch* Rhs)
	{
		return FCString::Stricmp(*Lhs->Comment, *Rhs->Comment);
	};

	auto GetCompareFunc = [&](const FName& ColumnId)
	{
		if (ColumnId == PlasticSourceControlBranchesListViewColumn::Name::Id())
		{
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>(CompareNames);
		}
		else if (ColumnId == PlasticSourceControlBranchesListViewColumn::Repository::Id())
		{
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>(CompareRepository);
		}
		else if (ColumnId == PlasticSourceControlBranchesListViewColumn::CreatedBy::Id())
		{
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>(CompareCreatedBy);
		}
		else if (ColumnId == PlasticSourceControlBranchesListViewColumn::Date::Id())
		{
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>(CompareDate);
		}
		else if (ColumnId == PlasticSourceControlBranchesListViewColumn::Comment::Id())
		{
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>(CompareComment);
		}
		else
		{
			checkNoEntry();
			return TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)>();
		};
	};

	TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)> PrimaryCompare = GetCompareFunc(PrimarySortedColumn);
	TFunction<int32(const FPlasticSourceControlBranch*, const FPlasticSourceControlBranch*)> SecondaryCompare;
	if (!SecondarySortedColumn.IsNone())
	{
		SecondaryCompare = GetCompareFunc(SecondarySortedColumn);
	}

	if (PrimarySortMode == EColumnSortMode::Ascending)
	{
		// NOTE: StableSort() would give a better experience when the sorted columns(s) has the same values and new values gets added, but it is slower
		//       with large changelists (7600 items was about 1.8x slower in average measured with Unreal Insight). Because this code runs in the main
		//       thread and can be invoked a lot, the trade off went if favor of speed.
		BranchRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlBranchPtr& Lhs, const FPlasticSourceControlBranchPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get()));
			if (Result < 0)
			{
				return true;
			}
			else if (Result > 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get())) > 0;
			}
		});
	}
	else
	{
		BranchRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlBranchPtr& Lhs, const FPlasticSourceControlBranchPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get()));
			if (Result > 0)
			{
				return true;
			}
			else if (Result < 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlBranch*>(Lhs.Get()), static_cast<FPlasticSourceControlBranch*>(Rhs.Get())) > 0;
			}
		});
	}
}

TArray<FString> SPlasticSourceControlBranchesWidget::GetSelectedBranches()
{
	TArray<FString> SelectedBranches;

	for (const FPlasticSourceControlBranchRef& BranchRef : BranchesListView->GetSelectedItems())
	{
		SelectedBranches.Add(BranchRef->Name);
	}

	return SelectedBranches;
}

TSharedPtr<SWidget> SPlasticSourceControlBranchesWidget::OnOpenContextMenu()
{
	const TArray<FString> SelectedBranches = GetSelectedBranches();
	const FString& SelectedBranch = SelectedBranches.Num() == 1 ? SelectedBranches[0] : FString();
	if (SelectedBranches.Num() == 0)
	{
		return nullptr;
	}
	const bool bSingleSelection = !SelectedBranch.IsEmpty();
	const bool bSingleNotCurrent = bSingleSelection && (SelectedBranch != WorkspaceSelector);

	const bool bMergeXml = FPlasticSourceControlModule::Get().GetProvider().GetPlasticScmVersion() >= PlasticSourceControlVersions::MergeXml;

	static const FText SelectASingleBranchTooltip(LOCTEXT("SelectASingleBranchTooltip", "Select a single branch."));
	static const FText SelectADifferentBranchTooltip(LOCTEXT("SelectADifferentBranchTooltip", "Select a branch that is not the current one."));
	static const FText UpdateUVCSTooltip(LOCTEXT("MergeBranchXmlTooltip", "Update Unity Version Control (PlasticSCM) to 11.0.16.7726 or later."));

	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName = "PlasticSourceControl.BranchesContextMenu";
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	// Build up the menu
	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");

	const FText CreateChildBranchTooltip(FText::Format(LOCTEXT("CreateChildBranchTooltip", "Create a child branch from {0}"),
		FText::FromString(SelectedBranch)));
	const FText& CreateChildBranchTooltipDynamic = bSingleSelection ? CreateChildBranchTooltip : SelectASingleBranchTooltip;
	Section.AddMenuEntry(
		"CreateChildBranch",
		LOCTEXT("CreateChildBranch", "Create child branch..."),
		CreateChildBranchTooltipDynamic,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnCreateBranchClicked, SelectedBranch),
			FCanExecuteAction::CreateLambda([bSingleSelection]() { return bSingleSelection; })
		)
	);

	const FText SwitchToBranchTooltip(FText::Format(LOCTEXT("SwitchToBranchTooltip", "Switch the workspace to the branch {0}"),
		FText::FromString(SelectedBranch)));
	const FText& SwitchToBranchTooltipDynamic =
		bSingleNotCurrent ? SwitchToBranchTooltip :
		bSingleSelection ? SelectADifferentBranchTooltip : SelectASingleBranchTooltip;
	Section.AddMenuEntry(
		"SwitchToBranch",
		LOCTEXT("SwitchToBranch", "Switch workspace to this branch"),
		SwitchToBranchTooltipDynamic,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnSwitchToBranchClicked, SelectedBranch),
			FCanExecuteAction::CreateLambda([bSingleNotCurrent]() { return bSingleNotCurrent; })
		)
	);

	Section.AddSeparator("PlasticSeparator1");

	const FText MergeBranchTooltip(FText::Format(LOCTEXT("MergeBranchTooltip", "Merge this branch {0} into the current branch {1}"),
		FText::FromString(SelectedBranch), FText::FromString(WorkspaceSelector)));
	const FText& MergeBranchTooltipDynamic =
		!bMergeXml ? UpdateUVCSTooltip :
		bSingleNotCurrent ? MergeBranchTooltip :
		bSingleSelection ? SelectADifferentBranchTooltip : SelectASingleBranchTooltip;
	Section.AddMenuEntry(
		"MergeBranch",
		LOCTEXT("MergeBranch", "Merge from this branch..."),
		MergeBranchTooltipDynamic,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnMergeBranchClicked, SelectedBranch),
			FCanExecuteAction::CreateLambda([bMergeXml, bSingleNotCurrent]() { return bMergeXml && bSingleNotCurrent; })
		)
	);

	Section.AddSeparator("PlasticSeparator2");

	const FText RenameBranchTooltip(FText::Format(LOCTEXT("RenameBranchTooltip", "Rename the branch {0}"),
		FText::FromString(SelectedBranch)));
	const FText& RenameBranchTooltipDynamic = bSingleSelection ? RenameBranchTooltip : SelectASingleBranchTooltip;
	Section.AddMenuEntry(
		"RenameBranch",
		LOCTEXT("RenameBranch", "Rename..."),
		RenameBranchTooltipDynamic,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnRenameBranchClicked, SelectedBranch),
			FCanExecuteAction::CreateLambda([this, bSingleSelection]() { return bSingleSelection; })
		)
	);
	const FText DeleteBranchTooltip = bSingleSelection ?
		FText::Format(LOCTEXT("DeleteBranchTooltip", "Delete the branch {0}"), FText::FromString(SelectedBranch)) :
		LOCTEXT("DeleteBranchesTooltip", "Delete the selected branches.");
	Section.AddMenuEntry(
		"DeleteBranch",
		LOCTEXT("DeleteBranch", "Delete"),
		DeleteBranchTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnDeleteBranchesClicked, SelectedBranches),
			FCanExecuteAction()
		)
	);

	return ToolMenus->GenerateWidget(Menu);
}


TSharedPtr<SWindow> SPlasticSourceControlBranchesWidget::CreateDialogWindow(FText&& InTitle)
{
	return SNew(SWindow)
		.Title(InTitle)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea);
}

void SPlasticSourceControlBranchesWidget::OpenDialogWindow(TSharedPtr<SWindow>& InDialogWindowPtr)
{
	InDialogWindowPtr->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnDialogClosed));

	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
	FSlateApplication::Get().AddModalWindow(InDialogWindowPtr.ToSharedRef(), RootWindow);
}

void SPlasticSourceControlBranchesWidget::OnDialogClosed(const TSharedRef<SWindow>& InWindow)
{
	DialogWindowPtr = nullptr;
}

void SPlasticSourceControlBranchesWidget::OnCreateBranchClicked(FString InParentBranchName)
{
	// Create the branch modal dialog window (the frame for the content)
	DialogWindowPtr = CreateDialogWindow(LOCTEXT("PlasticCreateBranchTitle", "Create Branch"));

	// Setup its content widget, specific to the CreateBranch operation
	DialogWindowPtr->SetContent(SNew(SPlasticSourceControlCreateBranch)
		.BranchesWidget(SharedThis(this))
		.ParentWindow(DialogWindowPtr)
		.ParentBranchName(InParentBranchName));

	OpenDialogWindow(DialogWindowPtr);
}

void SPlasticSourceControlBranchesWidget::CreateBranch(const FString& InParentBranchName, const FString& InNewBranchName, const FString& InNewBranchComment, const bool bInSwitchWorkspace)
{
	if (!Notification.IsInProgress())
	{
		// Find and Unlink all loaded packages in Content directory to allow to update them
		PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

		// Launch a custom "CreateBranch" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticCreateBranch, ESPMode::ThreadSafe> CreateBranchOperation = ISourceControlOperation::Create<FPlasticCreateBranch>();
		CreateBranchOperation->BranchName = InParentBranchName + TEXT("/") + InNewBranchName;
		CreateBranchOperation->Comment = InNewBranchComment;
		const ECommandResult::Type Result = Provider.Execute(CreateBranchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnCreateBranchOperationComplete, bInSwitchWorkspace));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(CreateBranchOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(CreateBranchOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlBranchesWidget::OnSwitchToBranchClicked(FString InBranchName)
{
	if (!Notification.IsInProgress())
	{
		// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them. Saving and checking out these assets will make the switch to the branch fail.
		PackageUtils::SaveDirtyPackages();

		// Find and Unlink all loaded packages in Content directory to allow to update them
		PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

		// Launch a custom "Switch" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToBranchOperation = ISourceControlOperation::Create<FPlasticSwitch>();
		SwitchToBranchOperation->BranchName = InBranchName;
		const ECommandResult::Type Result = Provider.Execute(SwitchToBranchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnSwitchToBranchOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(SwitchToBranchOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(SwitchToBranchOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlBranchesWidget::OnMergeBranchClicked(FString InBranchName)
{
	const FText MergeBranchQuestion = FText::Format(LOCTEXT("MergeBranchDialog", "Merge branch {0} into the current branch {1}?"), FText::FromString(InBranchName), FText::FromString(WorkspaceSelector));
	const EAppReturnType::Type Choice = FMessageDialog::Open(
		EAppMsgCategory::Info,
		EAppMsgType::YesNo, MergeBranchQuestion
		, LOCTEXT("MergeBranchTitle", "Merge Branch?")
	);
	if (Choice == EAppReturnType::Yes)
	{
		if (!Notification.IsInProgress())
		{
			// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them. Saving and checking out these assets might make the merge of the branch fail.
			PackageUtils::SaveDirtyPackages();

			// Find and Unlink all loaded packages in Content directory to allow to update them
			PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

			// Launch a custom "Merge" operation
			FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
			TSharedRef<FPlasticMergeBranch, ESPMode::ThreadSafe> MergeBranchOperation = ISourceControlOperation::Create<FPlasticMergeBranch>();
			MergeBranchOperation->BranchName = InBranchName;
			const ECommandResult::Type Result = Provider.Execute(MergeBranchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnMergeBranchOperationComplete));
			if (Result == ECommandResult::Succeeded)
			{
				// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
				Notification.DisplayInProgress(MergeBranchOperation->GetInProgressString());
				StartRefreshStatus();
			}
			else
			{
				// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
				FNotification::DisplayFailure(MergeBranchOperation.Get());
			}
		}
		else
		{
			FMessageLog SourceControlLog("SourceControl");
			SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
			SourceControlLog.Notify();
		}
	}
}

void SPlasticSourceControlBranchesWidget::OnRenameBranchClicked(FString InBranchName)
{
	// Create the branch modal dialog window (the frame for the content)
	DialogWindowPtr = CreateDialogWindow(LOCTEXT("PlasticRenameBranchTitle", "Rename Branch"));

	// Setup its content widget, specific to the RenameBranch operation
	DialogWindowPtr->SetContent(SNew(SPlasticSourceControlRenameBranch)
		.BranchesWidget(SharedThis(this))
		.ParentWindow(DialogWindowPtr)
		.OldBranchName(InBranchName));

	OpenDialogWindow(DialogWindowPtr);
}

void SPlasticSourceControlBranchesWidget::RenameBranch(const FString& InOldBranchName, const FString& InNewBranchName)
{
	if (!Notification.IsInProgress())
	{
		// Launch a custom "RenameBranch" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticRenameBranch, ESPMode::ThreadSafe> RenameBranchOperation = ISourceControlOperation::Create<FPlasticRenameBranch>();
		RenameBranchOperation->OldName = InOldBranchName;
		RenameBranchOperation->NewName = InNewBranchName;
		const ECommandResult::Type Result = Provider.Execute(RenameBranchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnRenameBranchOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(RenameBranchOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(RenameBranchOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlBranchesWidget::OnDeleteBranchesClicked(TArray<FString> InBranchNames)
{
	// Create the branch modal dialog window (the frame for the content)
	DialogWindowPtr = CreateDialogWindow(LOCTEXT("PlasticDeleteBranchesTitle", "Delete Branches"));

	// Setup its content widget, specific to the DeleteBranches operation
	DialogWindowPtr->SetContent(SNew(SPlasticSourceControlDeleteBranches)
		.BranchesWidget(SharedThis(this))
		.ParentWindow(DialogWindowPtr)
		.BranchNames(InBranchNames));

	OpenDialogWindow(DialogWindowPtr);
}

void SPlasticSourceControlBranchesWidget::DeleteBranches(const TArray<FString>& InBranchNames)
{
	if (!Notification.IsInProgress())
	{
		// Launch a custom "DeleteBranches" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticDeleteBranches, ESPMode::ThreadSafe> DeleteBranchesOperation = ISourceControlOperation::Create<FPlasticDeleteBranches>();
		DeleteBranchesOperation->BranchNames = InBranchNames;
		const ECommandResult::Type Result = Provider.Execute(DeleteBranchesOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnDeleteBranchesOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(DeleteBranchesOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(DeleteBranchesOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlBranchesWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	// Detect transitions of the source control being available/unavailable. Ex: When the user changes the source control in UI, the provider gets selected,
	// but it is not connected/available until the user accepts the settings. The source control doesn't have callback for availability and we want to refresh everything
	// once it gets available.
	if (ISourceControlModule::Get().IsEnabled() && !bSourceControlAvailable && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		bSourceControlAvailable = true;
		bShouldRefresh = true;
	}

	if (bShouldRefresh)
	{
		RequestBranchesRefresh();
		bShouldRefresh = false;
	}

	if (bIsRefreshing)
	{
		TickRefreshStatus(InDeltaTime);
	}
}

bool SPlasticSourceControlBranchesWidget::IsBranchNameValid(const FString& InBranchName)
{
	// Branch name cannot contain any of the following characters:
	// Note: tabs are technically not forbidden in branch names, but having one at the end doesn't work as expected
	// (it is trimmed at creation, so the switch to the new branch fails)
	static const FString BranchNameInvalidChars(TEXT("@#/:\"?'\n\r\t"));

	for (TCHAR Char : InBranchName)
	{
		int32 Index;
		if (BranchNameInvalidChars.FindChar(Char, Index))
		{
			return false;
		}
	}

	return true;
}

void SPlasticSourceControlBranchesWidget::StartRefreshStatus()
{
	if (!bIsRefreshing)
	{
		bIsRefreshing = true;
		RefreshStatusStartSecs = FPlatformTime::Seconds();
	}
}

void SPlasticSourceControlBranchesWidget::TickRefreshStatus(double InDeltaTime)
{
	const int32 RefreshStatusTimeElapsed = static_cast<int32>(FPlatformTime::Seconds() - RefreshStatusStartSecs);
	RefreshStatus = FText::Format(LOCTEXT("PlasticSourceControl_RefreshBranches", "Refreshing branches... ({0} s)"), FText::AsNumber(RefreshStatusTimeElapsed));
}

void SPlasticSourceControlBranchesWidget::EndRefreshStatus()
{
	bIsRefreshing = false;
	RefreshStatus = FText::GetEmpty();
}

void SPlasticSourceControlBranchesWidget::RequestBranchesRefresh()
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	StartRefreshStatus();

	TSharedRef<FPlasticGetBranches, ESPMode::ThreadSafe> GetBranchesOperation = ISourceControlOperation::Create<FPlasticGetBranches>();
	if (FromDateInDays > -1)
	{
		GetBranchesOperation->FromDate = FDateTime::Now() - FTimespan::FromDays(FromDateInDays);
	}

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	Provider.Execute(GetBranchesOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlBranchesWidget::OnGetBranchesOperationComplete));
}

void SPlasticSourceControlBranchesWidget::OnGetBranchesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::OnGetBranchesOperationComplete);

	TSharedRef<FPlasticGetBranches, ESPMode::ThreadSafe> OperationGetBranches = StaticCastSharedRef<FPlasticGetBranches>(InOperation);
	SourceControlBranches = MoveTemp(OperationGetBranches->Branches);

	WorkspaceSelector = FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector();

	EndRefreshStatus();
	OnRefreshUI();
}

void SPlasticSourceControlBranchesWidget::OnCreateBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, const bool bInSwitchWorkspace)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::OnCreateBranchOperationComplete);

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);

	if (InResult == ECommandResult::Succeeded)
	{
		if (bInSwitchWorkspace)
		{
			TSharedRef<FPlasticCreateBranch, ESPMode::ThreadSafe> CreateBranchOperation = StaticCastSharedRef<FPlasticCreateBranch>(InOperation);
			OnSwitchToBranchClicked(CreateBranchOperation->BranchName);
		}
		else
		{
			// Ask for a full refresh of the list of branches (and don't call EndRefreshStatus() yet)
			bShouldRefresh = true;
		}
	}
	else
	{
		EndRefreshStatus();
	}
}

void SPlasticSourceControlBranchesWidget::OnSwitchToBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::OnSwitchToBranchOperationComplete);

	// Reload packages that where updated by the SwitchToBranch operation (and the current map if needed)
	TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToBranchOperation = StaticCastSharedRef<FPlasticSwitch>(InOperation);
	PackageUtils::ReloadPackages(SwitchToBranchOperation->UpdatedFiles);

	// Ask for a full refresh of the list of branches (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlBranchesWidget::OnMergeBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlBranchesWidget::OnMergeBranchOperationComplete);

	// Reload packages that where updated by the MergeBranch operation (and the current map if needed)
	TSharedRef<FPlasticMergeBranch, ESPMode::ThreadSafe> MergeBranchOperation = StaticCastSharedRef<FPlasticMergeBranch>(InOperation);
	PackageUtils::ReloadPackages(MergeBranchOperation->UpdatedFiles);

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);

	EndRefreshStatus();
}

void SPlasticSourceControlBranchesWidget::OnRenameBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	// Ask for a full refresh of the list of branches (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlBranchesWidget::OnDeleteBranchesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	// Ask for a full refresh of the list of branches (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlBranchesWidget::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	bSourceControlAvailable = NewProvider.IsAvailable(); // Check if it is connected.
	bShouldRefresh = true;

	if (&NewProvider != &OldProvider)
	{
		BranchRows.Reset();
		if (GetListView())
		{
			GetListView()->RequestListRefresh();
		}
	}
}

void SPlasticSourceControlBranchesWidget::SwitchToBranchWithConfirmation(const FString& InSelectedBranch)
{
	const FText SwitchToBranchQuestion = FText::Format(LOCTEXT("SwitchToBranchDialog", "Switch workspace to branch {0}?"), FText::FromString(InSelectedBranch));
	const EAppReturnType::Type Choice = FMessageDialog::Open(
		EAppMsgCategory::Info,
		EAppMsgType::YesNo, SwitchToBranchQuestion
		, LOCTEXT("SwitchToBranchTitle", "Switch Branch?")
	);
	if (Choice == EAppReturnType::Yes)
	{
		OnSwitchToBranchClicked(InSelectedBranch);
	}
}

void SPlasticSourceControlBranchesWidget::HandleSourceControlStateChanged()
{
	if (WorkspaceSelector != FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector())
	{
		bShouldRefresh = true;
	}
}

void SPlasticSourceControlBranchesWidget::OnItemDoubleClicked(FPlasticSourceControlBranchRef InBranch)
{
	// Double click switches to the selected branch (with a confirmation dialog)
	if (InBranch->Name != WorkspaceSelector)
	{
		SwitchToBranchWithConfirmation(InBranch->Name);
	}
}

FReply SPlasticSourceControlBranchesWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F5)
	{
		// Pressing F5 refreshes the list of branches
		bShouldRefresh = true;
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Pressing Enter switches to the selected branch (with a confirmation dialog)
		const TArray<FString> SelectedBranches = GetSelectedBranches();
		if (SelectedBranches.Num() == 1 && SelectedBranches[0] != WorkspaceSelector)
		{
			SwitchToBranchWithConfirmation(SelectedBranches[0]);
		}
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::F2)
	{
		// Pressing F2 renames the selected branches (with a dialog)
		const TArray<FString> SelectedBranches = GetSelectedBranches();
		if (SelectedBranches.Num() == 1)
		{
			const FString& SelectedBranch = SelectedBranches[0];
			OnRenameBranchClicked(SelectedBranch);
		}
		return FReply::Handled();
	}
	else if ((InKeyEvent.GetKey() == EKeys::Delete) || (InKeyEvent.GetKey() == EKeys::BackSpace))
	{
		// Pressing Delete or BackSpace deletes the selected branches (with a confirmation dialog)
		const TArray<FString> SelectedBranches = GetSelectedBranches();
		if (SelectedBranches.Num() > 0)
		{
			OnDeleteBranchesClicked(SelectedBranches);
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
