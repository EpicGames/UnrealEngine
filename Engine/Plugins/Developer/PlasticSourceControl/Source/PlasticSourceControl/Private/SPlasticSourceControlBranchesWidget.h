// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Notification.h"

#include "Misc/TextFilter.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"

typedef TSharedRef<class FPlasticSourceControlBranch, ESPMode::ThreadSafe> FPlasticSourceControlBranchRef;

class SSearchBox;
class SWindow;

// Widget displaying the list of branches in the tab window, see FPlasticSourceControlBranchesWindow
class SPlasticSourceControlBranchesWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPlasticSourceControlBranchesWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	static bool IsBranchNameValid(const FString& InBranchName);

	void CreateBranch(const FString& InParentBranchName, const FString& InNewBranchName, const FString& InNewBranchComment, const bool bInSwitchWorkspace);
	void RenameBranch(const FString& InOldBranchName, const FString& InNewBranchName);
	void DeleteBranches(const TArray<FString>& InBranchNames);

private:
	TSharedRef<SWidget> CreateToolBar();
	TSharedRef<SWidget> CreateContentPanel();

	TSharedRef<ITableRow> OnGenerateRow(FPlasticSourceControlBranchRef InBranch, const TSharedRef<STableViewBase>& OwnerTable);
	void OnHiddenColumnsListChanged();

	void OnSearchTextChanged(const FText& InFilterText);
	void PopulateItemSearchStrings(const FPlasticSourceControlBranch& InItem, TArray<FString>& OutStrings);

	TSharedRef<SWidget> BuildFromDateDropDownMenu();
	void OnFromDateChanged(int32 InFromDateInDays);

	void OnRefreshUI();

	EColumnSortPriority::Type GetColumnSortPriority(const FName InColumnId) const;
	EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

	void SortBranchView();

	TArray<FString> GetSelectedBranches();

	TSharedPtr<SWidget> OnOpenContextMenu();

	TSharedPtr<SWindow> CreateDialogWindow(FText&& InTitle);
	void OpenDialogWindow(TSharedPtr<SWindow>& InDialogWindowPtr);
	void OnDialogClosed(const TSharedRef<SWindow>& InWindow);

	void OnCreateBranchClicked(FString InParentBranchName);
	void OnSwitchToBranchClicked(FString InBranchName);
	void OnMergeBranchClicked(FString InBranchName);
	void OnRenameBranchClicked(FString InBranchName);
	void OnDeleteBranchesClicked(TArray<FString> InBranchNames);

	void StartRefreshStatus();
	void TickRefreshStatus(double InDeltaTime);
	void EndRefreshStatus();

	void RequestBranchesRefresh();

	/** Source control callbacks */
	void OnGetBranchesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnCreateBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, const bool bInSwitchWorkspace);
	void OnSwitchToBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnMergeBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnRenameBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnDeleteBranchesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

	/** Delegate handler for when source control state changes */
	void HandleSourceControlStateChanged();

	SListView<FPlasticSourceControlBranchRef>* GetListView() const
	{
		return BranchesListView.Get();
	}

	void SwitchToBranchWithConfirmation(const FString& InSelectedBranch);

	/** Double click to switch to the branch */
	void OnItemDoubleClicked(FPlasticSourceControlBranchRef InBranch);

	/** Interpret F5, Enter and Delete keys */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TSharedPtr<SSearchBox> BranchSearchBox;

	FName PrimarySortedColumn;
	FName SecondarySortedColumn;
	EColumnSortMode::Type PrimarySortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type SecondarySortMode = EColumnSortMode::None;

	TArray<FName> HiddenColumnsList;

	bool bShouldRefresh = false;
	bool bSourceControlAvailable = false;

	FText RefreshStatus;
	bool bIsRefreshing = false;
	double RefreshStatusStartSecs;

	FString WorkspaceSelector;

	/** Ongoing notification for a long-running asynchronous source control operation, if any */
	FNotification Notification;

	TSharedPtr<SListView<FPlasticSourceControlBranchRef>> BranchesListView;
	TSharedPtr<TTextFilter<const FPlasticSourceControlBranch&>> SearchTextFilter;

	TMap<int32, FText> FromDateInDaysValues;
	int32 FromDateInDays = 30;

	TArray<FPlasticSourceControlBranchRef> SourceControlBranches; // Full list from source (filtered by date)
	TArray<FPlasticSourceControlBranchRef> BranchRows; // Filtered list to display based on the search text filter

	/** Delegate handle for the HandleSourceControlStateChanged function callback */
	FDelegateHandle SourceControlStateChangedDelegateHandle;

	/** The dialog Window that opens when the user click on any context menu entry */
	TSharedPtr<SWindow> DialogWindowPtr;
};
