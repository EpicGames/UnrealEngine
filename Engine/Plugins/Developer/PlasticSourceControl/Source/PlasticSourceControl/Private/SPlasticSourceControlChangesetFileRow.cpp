// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlChangesetFileRow.h"

#include "PlasticSourceControlState.h"
#include "PlasticSourceControlUtils.h"

#include "Misc/Paths.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Text/STextBlock.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlChangesetFileWindow"

FName PlasticSourceControlChangesetFilesListViewColumn::Icon::Id() { return TEXT("Icon"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Icon::GetDisplayText() { return LOCTEXT("Icon_Column", "Revision Control Status"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Icon::GetToolTipText() { return LOCTEXT("Icon_Column_Tooltip", "Icon displaying the type of change"); }

FName PlasticSourceControlChangesetFilesListViewColumn::Name::Id() { return TEXT("Name"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Name::GetDisplayText() { return LOCTEXT("Name_Column", "Name"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Name::GetToolTipText() { return LOCTEXT("Name_Column_Tooltip", "Name of the file"); }

FName PlasticSourceControlChangesetFilesListViewColumn::Path::Id() { return TEXT("Path"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Path::GetDisplayText() { return LOCTEXT("Path_Column", "Path"); }
FText PlasticSourceControlChangesetFilesListViewColumn::Path::GetToolTipText() { return LOCTEXT("Path_Column_Tooltip", "Path of the file relative to the workspace"); }

void SPlasticSourceControlChangesetFileRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	FileToVisualize = InArgs._FileToVisualize.Get();
	HighlightText = InArgs._HighlightText;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> GetSCCFileWidget(FPlasticSourceControlState* InFileState)
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush("ContentBrowser.ColumnViewAssetIcon");

	// Make icon overlays (eg, SCC and dirty status) a reasonable size in relation to the icon size (note: it is assumed this icon is square)
	const float ICON_SCALING_FACTOR = 0.7f;
	const float IconOverlaySize = static_cast<float>(IconBrush->ImageSize.X * ICON_SCALING_FACTOR);

	return SNew(SOverlay)
		// The actual icon
		+SOverlay::Slot()
		[
			SNew(SImage)
			.Image(IconBrush)
		]
		// Source control state
		+SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SBox)
			.WidthOverride(IconOverlaySize)
			.HeightOverride(IconOverlaySize)
			[
				SNew(SLayeredImage, InFileState->GetIcon())
				.ToolTipText(InFileState->GetDisplayTooltip())
			]
		];
}


TSharedRef<SWidget> SPlasticSourceControlChangesetFileRow::GenerateWidgetForColumn(const FName& InColumnId)
{
	if (InColumnId == PlasticSourceControlChangesetFilesListViewColumn::Icon::Id())
	{
		return SNew(SBox)
			.WidthOverride(16.f) // Small Icons are usually 16x16
			.ToolTipText(FileToVisualize->ToText())
			.HAlign(HAlign_Center)
			[
				GetSCCFileWidget(FileToVisualize)
			];
	}
	else if (InColumnId == PlasticSourceControlChangesetFilesListViewColumn::Name::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FPaths::GetBaseFilename(FileToVisualize->LocalFilename, true))) // Just the name without its path or extension
			.ToolTipText(FText::FromString(FPaths::GetCleanFilename(FileToVisualize->LocalFilename))) // Name with extension
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlChangesetFilesListViewColumn::Path::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(FPaths::GetBaseFilename(FileToVisualize->LocalFilename, false))) // Relative Path without the extension
			.ToolTipText(FText::FromString(FileToVisualize->LocalFilename))
			.HighlightText(HighlightText);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
