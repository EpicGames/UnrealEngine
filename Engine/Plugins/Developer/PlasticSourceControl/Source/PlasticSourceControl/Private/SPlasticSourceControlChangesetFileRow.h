// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

typedef TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStateRef;
typedef TSharedPtr<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStatePtr;

class FPlasticSourceControlState;

/** Lists the unique columns used in the list view displaying Files in the selected changeset. */
namespace PlasticSourceControlChangesetFilesListViewColumn
{
	/** The Icon displaying the type of Change column. */
	namespace Icon // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The File Name column. */
	namespace Name // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The File Path column. */
	namespace Path // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

} // namespace PlasticSourceControlChangesetFilesListViewColumn

class SPlasticSourceControlChangesetFileRow : public SMultiColumnTableRow<FPlasticSourceControlStateRef>
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlChangesetFileRow)
		: _FileToVisualize(nullptr)
		, _HighlightText()
	{
	}
		SLATE_ARGUMENT(FPlasticSourceControlStatePtr, FileToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:
	/**
	* Construct a row child widgets of the ListView.
	*
	* @param InArgs Parameters including the File to visualize in this row.
	* @param InOwner The owning ListView.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

private:
	/** The File that we are visualizing in this row. */
	FPlasticSourceControlState* FileToVisualize;

	/** The search text to highlight if any */
	TAttribute<FText> HighlightText;
};
