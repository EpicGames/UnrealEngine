// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlChangesetRow.h"

#include "PlasticSourceControlChangeset.h"
#include "PlasticSourceControlUtils.h"

#include "Widgets/Text/STextBlock.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlChangesetWindow"

FName PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id() { return TEXT("ChangesetId"); }
FText PlasticSourceControlChangesetsListViewColumn::ChangesetId::GetDisplayText() { return LOCTEXT("ChangesetId_Column", "Name"); }
FText PlasticSourceControlChangesetsListViewColumn::ChangesetId::GetToolTipText() { return LOCTEXT("ChangesetId_Column_Tooltip", "Id of the changeset"); }

FName PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id() { return TEXT("CreatedBy"); }
FText PlasticSourceControlChangesetsListViewColumn::CreatedBy::GetDisplayText() { return LOCTEXT("CreatedBy_Column", "Created by"); }
FText PlasticSourceControlChangesetsListViewColumn::CreatedBy::GetToolTipText() { return LOCTEXT("CreatedBy_Column_Tooltip", "Creator of the changeset"); }

FName PlasticSourceControlChangesetsListViewColumn::Date::Id() { return TEXT("Date"); }
FText PlasticSourceControlChangesetsListViewColumn::Date::GetDisplayText() { return LOCTEXT("Date_Column", "Creation date"); }
FText PlasticSourceControlChangesetsListViewColumn::Date::GetToolTipText() { return LOCTEXT("Date_Column_Tooltip", "Date of creation of the changeset"); }

FName PlasticSourceControlChangesetsListViewColumn::Comment::Id() { return TEXT("Comment"); }
FText PlasticSourceControlChangesetsListViewColumn::Comment::GetDisplayText() { return LOCTEXT("Comment_Column", "Comment"); }
FText PlasticSourceControlChangesetsListViewColumn::Comment::GetToolTipText() { return LOCTEXT("Comment_Column_Tooltip", "Comment describing the changeset"); }

FName PlasticSourceControlChangesetsListViewColumn::Branch::Id() { return TEXT("Branch"); }
FText PlasticSourceControlChangesetsListViewColumn::Branch::GetDisplayText() { return LOCTEXT("Branch_Column", "Branch"); }
FText PlasticSourceControlChangesetsListViewColumn::Branch::GetToolTipText() { return LOCTEXT("Branch_Column_Tooltip", "Branch where the changeset was created"); }

void SPlasticSourceControlChangesetRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	ChangesetToVisualize = InArgs._ChangesetToVisualize.Get();
	bIsCurrentChangeset = InArgs._bIsCurrentChangeset;
	HighlightText = InArgs._HighlightText;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> SPlasticSourceControlChangesetRow::GenerateWidgetForColumn(const FName& InColumnId)
{
	const FSlateFontInfo FontInfo = bIsCurrentChangeset ? FAppStyle::GetFontStyle("BoldFont") : FAppStyle::GetFontStyle("NormalFont");

	if (InColumnId == PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id())
	{
		return SNew(STextBlock)
			.Text(FText::AsNumber(ChangesetToVisualize->ChangesetId))
			.ToolTipText(FText::AsNumber(ChangesetToVisualize->ChangesetId))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(PlasticSourceControlUtils::UserNameToDisplayName(ChangesetToVisualize->CreatedBy)))
			.ToolTipText(FText::FromString(ChangesetToVisualize->CreatedBy))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlChangesetsListViewColumn::Date::Id())
	{
		return SNew(STextBlock)
			.Text(FText::AsDateTime(ChangesetToVisualize->Date))
			.ToolTipText(FText::AsDateTime(ChangesetToVisualize->Date))
			.Margin(FMargin(6.f, 1.f))
			.Font(FontInfo);
	}
	else if (InColumnId == PlasticSourceControlChangesetsListViewColumn::Comment::Id())
	{
		// Make each comment fit on a single line to preserve the table layout
		FString CommentOnOneLine = ChangesetToVisualize->Comment;
		CommentOnOneLine.ReplaceCharInline(TEXT('\n'), TEXT(' '), ESearchCase::CaseSensitive);

		return SNew(STextBlock)
			.Text(FText::FromString(MoveTemp(CommentOnOneLine)))
			.ToolTipText(FText::FromString(ChangesetToVisualize->Comment))
			.Margin(FMargin(6.f, 1.f))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlChangesetsListViewColumn::Branch::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(ChangesetToVisualize->Branch))
			.ToolTipText(FText::FromString(ChangesetToVisualize->Branch))
			.Margin(FMargin(6.f, 1.f))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Font(FontInfo)
			.HighlightText(HighlightText);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
