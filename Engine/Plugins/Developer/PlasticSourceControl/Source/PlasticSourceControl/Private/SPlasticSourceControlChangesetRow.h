// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedPtr<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetPtr;

class FPlasticSourceControlChangeset;

/** Lists the unique columns used in the list view displaying Changesets. */
namespace PlasticSourceControlChangesetsListViewColumn
{
	/** The Changeset ChangesetId column. */
	namespace ChangesetId // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The Changeset CreatedBy column. */
	namespace CreatedBy // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The Changeset Date column. */
	namespace Date // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The Changeset Comment column. */
	namespace Comment // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};


	/** The Changeset Branch column. */
	namespace Branch // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};
} // namespace PlasticSourceControlChangesetsListViewColumn

class SPlasticSourceControlChangesetRow : public SMultiColumnTableRow<FPlasticSourceControlChangesetRef>
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlChangesetRow)
		: _ChangesetToVisualize(nullptr)
		, _bIsCurrentChangeset(false)
		, _HighlightText()
	{
	}
		SLATE_ARGUMENT(FPlasticSourceControlChangesetPtr, ChangesetToVisualize)
		SLATE_ARGUMENT(bool, bIsCurrentChangeset)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:
	/**
	* Construct a row child widgets of the ListView.
	*
	* @param InArgs Parameters including the Changeset to visualize in this row.
	* @param InOwner The owning ListView.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

private:
	/** The Changeset that we are visualizing in this row. */
	FPlasticSourceControlChangeset* ChangesetToVisualize;

	/** True if this is the current changeset, to be highlighted on the list of changesets. */
	bool bIsCurrentChangeset;

	/** The search text to highlight if any */
	TAttribute<FText> HighlightText;
};
