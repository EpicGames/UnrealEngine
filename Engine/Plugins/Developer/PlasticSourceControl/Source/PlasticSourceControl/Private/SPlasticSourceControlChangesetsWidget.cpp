// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlChangesetsWidget.h"

#include "PlasticSourceControlChangeset.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlProjectSettings.h"
#include "PlasticSourceControlUtils.h"
#include "SPlasticSourceControlChangesetRow.h"
#include "SPlasticSourceControlChangesetFileRow.h"

#include "PackageUtils.h"

#include "AssetRegistry/AssetData.h"
#include "AssetToolsModule.h"
#include "DesktopPlatformModule.h"
#include "DiffUtils.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "ISourceControlModule.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "ScopedTransaction.h"
#include "SourceControlHelpers.h"
#include "SourceControlWindows.h"
#include "ToolMenus.h"
#include "ToolMenuContext.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/SBoxPanel.h"

#include "ISourceControlWindowsModule.h"
#include "Misc/ComparisonUtility.h"
#include "Selection.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlChangesetWindow"

void SPlasticSourceControlChangesetsWidget::Construct(const FArguments& InArgs)
{
	ISourceControlModule::Get().RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSourceControlProviderChanged));
	// register for any source control change to detect new local Changesets on check-in
	SourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SPlasticSourceControlChangesetsWidget::HandleSourceControlStateChanged));

	CurrentChangesetId = FPlasticSourceControlModule::Get().GetProvider().GetChangesetNumber();

	ChangesetsSearchTextFilter = MakeShared<TTextFilter<const FPlasticSourceControlChangeset&>>(TTextFilter<const FPlasticSourceControlChangeset&>::FItemToStringArray::CreateSP(this, &SPlasticSourceControlChangesetsWidget::PopulateItemSearchStrings));
	ChangesetsSearchTextFilter->OnChanged().AddSP(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsRefreshUI);

	FilesSearchTextFilter = MakeShared<TTextFilter<const FPlasticSourceControlState&>>(TTextFilter<const FPlasticSourceControlState&>::FItemToStringArray::CreateSP(this, &SPlasticSourceControlChangesetsWidget::PopulateItemSearchStrings));
	FilesSearchTextFilter->OnChanged().AddSP(this, &SPlasticSourceControlChangesetsWidget::OnFilesRefreshUI);

	FromDateInDaysValues.Add(TPair<int32, FText>(7, FText::FromString(TEXT("Last week"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(15, FText::FromString(TEXT("Last 15 days"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(30, FText::FromString(TEXT("Last month"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(91, FText::FromString(TEXT("Last 3 months"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(182, FText::FromString(TEXT("Last 6 months"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(365, FText::FromString(TEXT("Last year"))));
	FromDateInDaysValues.Add(TPair<int32, FText>(-1, FText::FromString(TEXT("All time"))));

	// Min/Max prevents making the Changeset Area too small
	const float ChangesetAreaRatio = 0.6f;
	const float FileAreaRatio = 1.0f - ChangesetAreaRatio;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot() // For the toolbar (Search box and Refresh button)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						CreateToolBar()
					]
					+SHorizontalBox::Slot()
					.MaxWidth(10.0f)
					[
						SNew(SSpacer)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(300.0f)
					[
						SAssignNew(ChangesetsSearchBox, SSearchBox)
						.HintText(LOCTEXT("SearchChangesets", "Search changesets"))
						.ToolTipText(LOCTEXT("PlasticChangesetsSearch_Tooltip", "Filter the list of changesets by keyword."))
						.OnTextChanged(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsSearchTextChanged)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(125.0f)
					.Padding(10.f, 0.f)
					[
						SNew(SComboButton)
						.ToolTipText(LOCTEXT("PlasticChangesetesDate_Tooltip", "Filter the list of changesets by date of creation."))
						.OnGetMenuContent(this, &SPlasticSourceControlChangesetsWidget::BuildFromDateDropDownMenu)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text_Lambda([this]() { return FromDateInDaysValues[FromDateInDays]; })
						]
					]
				]
				// TODO: add a button to update the workspace when the current changeset is not the last one of the branch!
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					// Button to open the Branches View
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 0.0f))
					.ToolTipText(LOCTEXT("PlasticBranchesWindowTooltip", "Open the Branches window."))
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked_Lambda([]()
						{
							FPlasticSourceControlModule::Get().GetBranchesWindow().OpenTab();
							return FReply::Handled();
						})
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SourceControl.Branch"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
							.Text(LOCTEXT("PlasticBranchesWindow", "Branches"))
						]
					]
				]
			]
		]
		+SVerticalBox::Slot() // The main content: the splitter with the list of changesets, and the list of files in the selected changeset
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			.ResizeMode(ESplitterResizeMode::FixedPosition)

			// Left slot: Changesets area.
			+SSplitter::Slot()
			.Resizable(true)
			.SizeRule(SSplitter::FractionOfParent)
			.Value(ChangesetAreaRatio)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					CreateChangesetsListView()
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.FillHeight(1.0f)
				[
					// Text to display when there is no changesets displayed
					SNew(STextBlock)
					.Text(LOCTEXT("NoChangeset", "There is no changeset to display."))
					.Visibility_Lambda([this]() { return SourceControlChangesets.Num() ? EVisibility::Collapsed : EVisibility::Visible; })
				]
			]

			// Right slot: Files associated to the selected changeset.
			+SSplitter::Slot()
			.Resizable(true)
			.SizeRule(SSplitter::FractionOfParent)
			.Value(FileAreaRatio)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(5.0f)
				.AutoHeight()
				[
					SAssignNew(FilesSearchBox, SSearchBox)
					.MinDesiredWidth(200.0f)
					.HintText(LOCTEXT("SearchFiles", "Search the files"))
					.ToolTipText(LOCTEXT("PlasticFilesSearch_Tooltip", "Filter the list of files changed by keyword."))
					.OnTextChanged(this, &SPlasticSourceControlChangesetsWidget::OnFilesSearchTextChanged)
				]
				+SVerticalBox::Slot()
				[
					CreateFilesListView()
				]
				+SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.FillHeight(1.0f)
				[
					// Text to display when there is no changeset selected
					SNew(STextBlock)
					.Text(LOCTEXT("NoChangesetSelected", "Select a changeset from the left panel to see its files."))
					.Visibility_Lambda([this]() { return SourceControlChangesets.Num() == 0 || SourceSelectedChangeset.IsValid() ? EVisibility::Collapsed : EVisibility::Visible; })
				]
			]
		]
		+SVerticalBox::Slot() // Status bar (Always visible)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(FMargin(0.f, 3.f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return RefreshStatus; })
					.Margin(FMargin(5.f, 0.f))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::AsNumber(CurrentChangesetId); })
					.ToolTipText(LOCTEXT("PlasticChangesetCurrent_Tooltip", "Current changeset."))
				]
			]
		]
	];
}

TSharedRef<SWidget> SPlasticSourceControlChangesetsWidget::CreateToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]() { bShouldRefresh = true; })),
		NAME_None,
		LOCTEXT("SourceControl_RefreshButton", "Refresh"),
		LOCTEXT("SourceControl_RefreshButton_Tooltip", "Refreshes changesets from revision control provider."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"));

	return ToolBarBuilder.MakeWidget();
}

// Inspired by Engine\Source\Editor\SourceControlWindows\Private\SSourceControlChangelists.cpp
// TSharedRef<SListView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateChangelistFilesView()
TSharedRef<SWidget> SPlasticSourceControlChangesetsWidget::CreateChangesetsListView()
{
	UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
	if (!Settings->bShowChangesetCreatedByColumn)
	{
		ChangesetsHiddenColumnsList.Add(PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id());
	}
	if (!Settings->bShowChangesetDateColumn)
	{
		ChangesetsHiddenColumnsList.Add(PlasticSourceControlChangesetsListViewColumn::Date::Id());
	}
	if (!Settings->bShowChangesetCommentColumn)
	{
		ChangesetsHiddenColumnsList.Add(PlasticSourceControlChangesetsListViewColumn::Comment::Id());
	}
	if (!Settings->bShowChangesetBranchColumn)
	{
		ChangesetsHiddenColumnsList.Add(PlasticSourceControlChangesetsListViewColumn::Branch::Id());
	}

	TSharedRef<SListView<FPlasticSourceControlChangesetRef>> ChangesetView = SNew(SListView<FPlasticSourceControlChangesetRef>)
		.ListItemsSource(&ChangesetRows)
		.OnGenerateRow(this, &SPlasticSourceControlChangesetsWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged(this, &SPlasticSourceControlChangesetsWidget::OnSelectionChanged)
		.OnContextMenuOpening(this, &SPlasticSourceControlChangesetsWidget::OnOpenChangesetContextMenu)
		.OnMouseButtonDoubleClick(this, &SPlasticSourceControlChangesetsWidget::OnItemDoubleClicked)
		.OnItemToString_Debug_Lambda([this](FPlasticSourceControlChangesetRef Changeset) { return FString::FromInt(Changeset->ChangesetId); })
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList(ChangesetsHiddenColumnsList)
			.OnHiddenColumnsListChanged(this, &SPlasticSourceControlChangesetsWidget::OnHiddenColumnsListChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id())
			.DefaultLabel(PlasticSourceControlChangesetsListViewColumn::ChangesetId::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetsListViewColumn::ChangesetId::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(0.6f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority, PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode, PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
			.DefaultLabel(PlasticSourceControlChangesetsListViewColumn::CreatedBy::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetsListViewColumn::CreatedBy::GetToolTipText())
			.FillWidth(2.5f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority, PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode, PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetsListViewColumn::Date::Id())
			.DefaultLabel(PlasticSourceControlChangesetsListViewColumn::Date::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetsListViewColumn::Date::GetToolTipText())
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority, PlasticSourceControlChangesetsListViewColumn::Date::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode, PlasticSourceControlChangesetsListViewColumn::Date::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetsListViewColumn::Comment::Id())
			.DefaultLabel(PlasticSourceControlChangesetsListViewColumn::Comment::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetsListViewColumn::Comment::GetToolTipText())
			.FillWidth(5.0f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority, PlasticSourceControlChangesetsListViewColumn::Comment::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode, PlasticSourceControlChangesetsListViewColumn::Comment::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetsListViewColumn::Branch::Id())
			.DefaultLabel(PlasticSourceControlChangesetsListViewColumn::Branch::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetsListViewColumn::Branch::GetToolTipText())
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority, PlasticSourceControlChangesetsListViewColumn::Branch::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode, PlasticSourceControlChangesetsListViewColumn::Branch::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged)
		);

	ChangesetsListView = ChangesetView;

	return ChangesetView;
}

TSharedRef<SWidget> SPlasticSourceControlChangesetsWidget::CreateFilesListView()
{
	// Note: array of file States, each with one Revision for Diffing (like for Files and ShelvedFiles in FPlasticSourceControlChangelist)
	TSharedRef<SListView<FPlasticSourceControlStateRef>> FilesView = SNew(SListView<FPlasticSourceControlStateRef>)
		.ListItemsSource(&FileRows)
		.OnGenerateRow(this, &SPlasticSourceControlChangesetsWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SPlasticSourceControlChangesetsWidget::OnOpenFileContextMenu)
		.OnMouseButtonDoubleClick(this, &SPlasticSourceControlChangesetsWidget::OnLocateFileClicked)
		.OnItemToString_Debug_Lambda([this](FPlasticSourceControlStateRef FileState) { return FileState->LocalFilename; })
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)

			+SHeaderRow::Column(PlasticSourceControlChangesetFilesListViewColumn::Icon::Id())
			.DefaultLabel(PlasticSourceControlChangesetFilesListViewColumn::Icon::GetDisplayText()) // Displayed in the drop down menu to show/hide columns
			.DefaultTooltip(PlasticSourceControlChangesetFilesListViewColumn::Icon::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillSized(18)
			.HeaderContentPadding(FMargin(0))
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortPriority, PlasticSourceControlChangesetFilesListViewColumn::Icon::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortMode, PlasticSourceControlChangesetFilesListViewColumn::Icon::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnFilesColumnSortModeChanged)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(1, 0)
				[
					SNew(SBox)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this](){ return GetFilesColumnSortMode(PlasticSourceControlChangesetFilesListViewColumn::Icon::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed; })
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("SourceControl.ChangelistsTab"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]
			]

			+SHeaderRow::Column(PlasticSourceControlChangesetFilesListViewColumn::Name::Id())
			.DefaultLabel(PlasticSourceControlChangesetFilesListViewColumn::Name::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetFilesListViewColumn::Name::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(0.7f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortPriority, PlasticSourceControlChangesetFilesListViewColumn::Name::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortMode, PlasticSourceControlChangesetFilesListViewColumn::Name::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnFilesColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlChangesetFilesListViewColumn::Path::Id())
			.DefaultLabel(PlasticSourceControlChangesetFilesListViewColumn::Path::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlChangesetFilesListViewColumn::Path::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortPriority, PlasticSourceControlChangesetFilesListViewColumn::Path::Id())
			.SortMode(this, &SPlasticSourceControlChangesetsWidget::GetFilesColumnSortMode, PlasticSourceControlChangesetFilesListViewColumn::Path::Id())
			.OnSort(this, &SPlasticSourceControlChangesetsWidget::OnFilesColumnSortModeChanged)
		);

	FilesListView = FilesView;

	return FilesView;
}

TSharedRef<ITableRow> SPlasticSourceControlChangesetsWidget::OnGenerateRow(FPlasticSourceControlChangesetRef InChangeset, const TSharedRef<STableViewBase>& OwnerTable)
{
	const bool bIsCurrentChangeset = InChangeset->ChangesetId == CurrentChangesetId;
	return SNew(SPlasticSourceControlChangesetRow, OwnerTable)
		.ChangesetToVisualize(InChangeset)
		.bIsCurrentChangeset(bIsCurrentChangeset)
		.HighlightText_Lambda([this]() { return ChangesetsSearchBox->GetText(); });
}

TSharedRef<ITableRow> SPlasticSourceControlChangesetsWidget::OnGenerateRow(FPlasticSourceControlStateRef InFile, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPlasticSourceControlChangesetFileRow, OwnerTable)
		.FileToVisualize(InFile)
		.HighlightText_Lambda([this]() { return FilesSearchBox->GetText(); });
}

void SPlasticSourceControlChangesetsWidget::OnHiddenColumnsListChanged()
{
	// Update and save config to reload it on the next Editor sessions
	if (ChangesetsListView && ChangesetsListView->GetHeaderRow())
	{
		UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
		Settings->bShowChangesetCreatedByColumn = true;
		Settings->bShowChangesetDateColumn = true;
		Settings->bShowChangesetCommentColumn = true;
		Settings->bShowChangesetBranchColumn = true;

		for (const FName& ColumnId : ChangesetsListView->GetHeaderRow()->GetHiddenColumnIds())
		{
			if (ColumnId == PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
			{
				Settings->bShowChangesetCreatedByColumn = false;
			}
			else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Date::Id())
			{
				Settings->bShowChangesetDateColumn = false;
			}
			else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Comment::Id())
			{
				Settings->bShowChangesetCommentColumn = false;
			}
			else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Branch::Id())
			{
				Settings->bShowChangesetBranchColumn = false;
			}
		}
		Settings->SaveConfig();
	}
}

void SPlasticSourceControlChangesetsWidget::OnChangesetsSearchTextChanged(const FText& InFilterText)
{
	ChangesetsSearchTextFilter->SetRawFilterText(InFilterText);
	ChangesetsSearchBox->SetError(ChangesetsSearchTextFilter->GetFilterErrorText());
}

void SPlasticSourceControlChangesetsWidget::OnFilesSearchTextChanged(const FText& InFilterText)
{
	FilesSearchTextFilter->SetRawFilterText(InFilterText);
	FilesSearchBox->SetError(FilesSearchTextFilter->GetFilterErrorText());
}

void SPlasticSourceControlChangesetsWidget::PopulateItemSearchStrings(const FPlasticSourceControlChangeset& InItem, TArray<FString>& OutStrings)
{
	InItem.PopulateSearchString(OutStrings);
}

void SPlasticSourceControlChangesetsWidget::PopulateItemSearchStrings(const FPlasticSourceControlState& InItem, TArray<FString>& OutStrings)
{
	InItem.PopulateSearchString(OutStrings);
}

void SPlasticSourceControlChangesetsWidget::OnFromDateChanged(int32 InFromDateInDays)
{
	FromDateInDays = InFromDateInDays;
	bShouldRefresh = true;
}

TSharedRef<SWidget> SPlasticSourceControlChangesetsWidget::BuildFromDateDropDownMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (const auto & FromDateInDaysValue : FromDateInDaysValues)
	{
		FUIAction MenuAction(FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnFromDateChanged, FromDateInDaysValue.Key));
		MenuBuilder.AddMenuEntry(FromDateInDaysValue.Value, FromDateInDaysValue.Value, FSlateIcon(), MenuAction);
	}

	return MenuBuilder.MakeWidget();
}

void SPlasticSourceControlChangesetsWidget::OnChangesetsRefreshUI()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::OnChangesetsRefreshUI);

	const int32 ItemCount = SourceControlChangesets.Num();
	ChangesetRows.Empty(ItemCount);
	for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
	{
		const FPlasticSourceControlChangesetRef& Item = SourceControlChangesets[ItemIndex];
		if (ChangesetsSearchTextFilter->PassesFilter(Item.Get()))
		{
			ChangesetRows.Emplace(Item);
		}
	}

	if (ChangesetsListView)
	{
		SortChangesetsView();
		ChangesetsListView->RequestListRefresh();

		// On changesets list refreshed, auto re-select the previously selected changeset if it still exists in the new list of SourceControlChangesets
		if (SourceSelectedChangeset.IsValid())
		{
			if (const FPlasticSourceControlChangesetRef* SelectedChangeset = SourceControlChangesets.FindByPredicate([this](const FPlasticSourceControlChangesetRef& Changeset) { return Changeset->ChangesetId == SourceSelectedChangeset->ChangesetId; }))
			{
				SourceSelectedChangeset = *SelectedChangeset;
				ChangesetsListView->SetSelection(*SelectedChangeset, ESelectInfo::Direct);
			}
			else
			{
				SourceSelectedChangeset.Reset();
			}
		}
		// Else, select the first changeset in the list
		if (!SourceSelectedChangeset.IsValid() && (ChangesetRows.Num() > 0))
		{
			SourceSelectedChangeset = ChangesetRows[0];
			ChangesetsListView->SetSelection(ChangesetRows[0], ESelectInfo::Direct);
		}
	}

	// And also refresh the list of files
	OnFilesRefreshUI();
}

void SPlasticSourceControlChangesetsWidget::OnFilesRefreshUI()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::OnFilesRefreshUI);

	if (SourceSelectedChangeset.IsValid())
	{
		const int32 ItemCount = SourceSelectedChangeset->Files.Num();
		FileRows.Empty(ItemCount);
		for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
		{
			const FPlasticSourceControlStateRef& Item = SourceSelectedChangeset->Files[ItemIndex];
			if (FilesSearchTextFilter->PassesFilter(Item.Get()))
			{
				FileRows.Emplace(Item);
			}
		}
	}
	else
	{
		FileRows.Reset();
	}

	if (FilesListView)
	{
		SortFilesView();
		FilesListView->RequestListRefresh();
	}
}

EColumnSortPriority::Type SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortPriority(const FName InColumnId) const
{
	if (InColumnId == ChangesetsPrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (InColumnId == ChangesetsSecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

EColumnSortMode::Type SPlasticSourceControlChangesetsWidget::GetChangesetsColumnSortMode(const FName InColumnId) const
{
	if (InColumnId == ChangesetsPrimarySortedColumn)
	{
		return ChangesetsPrimarySortMode;
	}
	else if (InColumnId == ChangesetsSecondarySortedColumn)
	{
		return ChangesetsSecondarySortMode;
	}

	return EColumnSortMode::None;
}

void SPlasticSourceControlChangesetsWidget::OnChangesetsColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	if (InSortPriority == EColumnSortPriority::Primary)
	{
		ChangesetsPrimarySortedColumn = InColumnId;
		ChangesetsPrimarySortMode = InSortMode;

		if (InColumnId == ChangesetsSecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			ChangesetsSecondarySortedColumn = FName();
			ChangesetsSecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (InSortPriority == EColumnSortPriority::Secondary)
	{
		ChangesetsSecondarySortedColumn = InColumnId;
		ChangesetsSecondarySortMode = InSortMode;
	}

	if (ChangesetsListView)
	{
		SortChangesetsView();
		ChangesetsListView->RequestListRefresh();
	}
}

void SPlasticSourceControlChangesetsWidget::SortChangesetsView()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::SortChangesetsView);

	if (ChangesetsPrimarySortedColumn.IsNone() || ChangesetRows.Num() == 0)
	{
		return; // No column selected for sorting or nothing to sort.
	}

	auto CompareChangesetIds = [](const FPlasticSourceControlChangeset* Lhs, const FPlasticSourceControlChangeset* Rhs)
	{
		return Lhs->ChangesetId < Rhs->ChangesetId ? -1 : (Lhs->ChangesetId == Rhs->ChangesetId ? 0 : 1);
	};

	auto CompareCreatedBys = [](const FPlasticSourceControlChangeset* Lhs, const FPlasticSourceControlChangeset* Rhs)
	{
		return FCString::Stricmp(*Lhs->CreatedBy, *Rhs->CreatedBy);
	};

	auto CompareDates = [](const FPlasticSourceControlChangeset* Lhs, const FPlasticSourceControlChangeset* Rhs)
	{
		return Lhs->Date < Rhs->Date ? -1 : (Lhs->Date == Rhs->Date ? 0 : 1);
	};

	auto CompareComments = [](const FPlasticSourceControlChangeset* Lhs, const FPlasticSourceControlChangeset* Rhs)
	{
		return FCString::Stricmp(*Lhs->Comment, *Rhs->Comment);
	};

	auto CompareBranches = [](const FPlasticSourceControlChangeset* Lhs, const FPlasticSourceControlChangeset* Rhs)
	{
		return UE::ComparisonUtility::CompareNaturalOrder(*Lhs->Branch, *Rhs->Branch);
	};

	auto GetCompareFunc = [&](const FName& ColumnId)
	{
		if (ColumnId == PlasticSourceControlChangesetsListViewColumn::ChangesetId::Id())
		{
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>(CompareChangesetIds);
		}
		else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::CreatedBy::Id())
		{
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>(CompareCreatedBys);
		}
		else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Date::Id())
		{
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>(CompareDates);
		}
		else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Comment::Id())
		{
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>(CompareComments);
		}
		else if (ColumnId == PlasticSourceControlChangesetsListViewColumn::Branch::Id())
		{
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>(CompareBranches);
		}
		else
		{
			checkNoEntry();
			return TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)>();
		};
	};

	TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)> PrimaryCompare = GetCompareFunc(ChangesetsPrimarySortedColumn);
	TFunction<int32(const FPlasticSourceControlChangeset*, const FPlasticSourceControlChangeset*)> SecondaryCompare;
	if (!ChangesetsSecondarySortedColumn.IsNone())
	{
		SecondaryCompare = GetCompareFunc(ChangesetsSecondarySortedColumn);
	}

	if (ChangesetsPrimarySortMode == EColumnSortMode::Ascending)
	{
		// NOTE: StableSort() would give a better experience when the sorted columns(s) has the same values and new values gets added, but it is slower
		//       with large changelists (7600 items was about 1.8x slower in average measured with Unreal Insight). Because this code runs in the main
		//       thread and can be invoked a lot, the trade off went if favor of speed.
		ChangesetRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlChangesetPtr& Lhs, const FPlasticSourceControlChangesetPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get()));
			if (Result < 0)
			{
				return true;
			}
			else if (Result > 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (ChangesetsSecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get())) > 0;
			}
		});
	}
	else
	{
		ChangesetRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlChangesetPtr& Lhs, const FPlasticSourceControlChangesetPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get()));
			if (Result > 0)
			{
				return true;
			}
			else if (Result < 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (ChangesetsSecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlChangeset*>(Lhs.Get()), static_cast<FPlasticSourceControlChangeset*>(Rhs.Get())) > 0;
			}
		});
	}
}

EColumnSortPriority::Type SPlasticSourceControlChangesetsWidget::GetFilesColumnSortPriority(const FName InColumnId) const
{
	if (InColumnId == FilesPrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (InColumnId == FilesSecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

EColumnSortMode::Type SPlasticSourceControlChangesetsWidget::GetFilesColumnSortMode(const FName InColumnId) const
{
	if (InColumnId == FilesPrimarySortedColumn)
	{
		return FilesPrimarySortMode;
	}
	else if (InColumnId == FilesSecondarySortedColumn)
	{
		return FilesSecondarySortMode;
	}

	return EColumnSortMode::None;
}

void SPlasticSourceControlChangesetsWidget::OnFilesColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	if (InSortPriority == EColumnSortPriority::Primary)
	{
		FilesPrimarySortedColumn = InColumnId;
		FilesPrimarySortMode = InSortMode;

		if (InColumnId == FilesSecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			FilesSecondarySortedColumn = FName();
			FilesSecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (InSortPriority == EColumnSortPriority::Secondary)
	{
		FilesSecondarySortedColumn = InColumnId;
		FilesSecondarySortMode = InSortMode;
	}

	if (FilesListView)
	{
		SortFilesView();
		FilesListView->RequestListRefresh();
	}
}

void SPlasticSourceControlChangesetsWidget::SortFilesView()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::SortFilesView);

	if (FilesPrimarySortedColumn.IsNone() || FileRows.Num() == 0)
	{
		return; // No column selected for sorting or nothing to sort.
	}

	auto CompareIcons = [](const FPlasticSourceControlState* Lhs, const FPlasticSourceControlState* Rhs)
	{
		const int32 LhsVal = static_cast<int32>(Lhs->WorkspaceState);
		const int32 RhsVal = static_cast<int32>(Rhs->WorkspaceState);
		return LhsVal < RhsVal ? -1 : (LhsVal == RhsVal ? 0 : 1);
	};

	auto CompareNames = [](const FPlasticSourceControlState* Lhs, const FPlasticSourceControlState* Rhs)
	{
		return FCString::Stricmp(*FPaths::GetBaseFilename(Lhs->LocalFilename), *FPaths::GetBaseFilename(Rhs->LocalFilename));
	};

	auto ComparePaths = [](const FPlasticSourceControlState* Lhs, const FPlasticSourceControlState* Rhs)
	{
		return FCString::Stricmp(*Lhs->LocalFilename, *Rhs->LocalFilename);
	};

	auto GetCompareFunc = [&](const FName& ColumnId)
	{
		if (ColumnId == PlasticSourceControlChangesetFilesListViewColumn::Icon::Id())
		{
			return TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)>(CompareIcons);
		}
		else if (ColumnId == PlasticSourceControlChangesetFilesListViewColumn::Name::Id())
		{
			return TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)>(CompareNames);
		}
		else if (ColumnId == PlasticSourceControlChangesetFilesListViewColumn::Path::Id())
		{
			return TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)>(ComparePaths);
		}
		else
		{
			checkNoEntry();
			return TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)>();
		};
	};

	TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)> PrimaryCompare = GetCompareFunc(FilesPrimarySortedColumn);
	TFunction<int32(const FPlasticSourceControlState*, const FPlasticSourceControlState*)> SecondaryCompare;
	if (!FilesSecondarySortedColumn.IsNone())
	{
		SecondaryCompare = GetCompareFunc(FilesSecondarySortedColumn);
	}

	if (FilesPrimarySortMode == EColumnSortMode::Ascending)
	{
		// NOTE: StableSort() would give a better experience when the sorted columns(s) has the same values and new values gets added, but it is slower
		//       with large changelists (7600 items was about 1.8x slower in average measured with Unreal Insight). Because this code runs in the main
		//       thread and can be invoked a lot, the trade off went if favor of speed.
		FileRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlStatePtr& Lhs, const FPlasticSourceControlStatePtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get()));
			if (Result < 0)
			{
				return true;
			}
			else if (Result > 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (FilesSecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get())) > 0;
			}
		});
	}
	else
	{
		FileRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlStatePtr& Lhs, const FPlasticSourceControlStatePtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get()));
			if (Result > 0)
			{
				return true;
			}
			else if (Result < 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (FilesSecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlState*>(Lhs.Get()), static_cast<FPlasticSourceControlState*>(Rhs.Get())) > 0;
			}
		});
	}
}

TSharedPtr<SWidget> SPlasticSourceControlChangesetsWidget::OnOpenChangesetContextMenu()
{
	const TArray<FPlasticSourceControlChangesetRef> SelectedChangesets = ChangesetsListView->GetSelectedItems();
	if (SelectedChangesets.Num() == 0)
	{
		return nullptr;
	}

	// Detect if all selected changesets are from the same branch
	bool bSingleBranchSelected = false;
	FPlasticSourceControlChangesetPtr SelectedChangeset;
	if (SelectedChangesets.Num() >= 1)
	{
		SelectedChangeset = SelectedChangesets[0];
		bSingleBranchSelected = true;
		for (int32 i = 1; i < SelectedChangesets.Num(); i++)
		{
			if (SelectedChangesets[i]->Branch != SelectedChangeset->Branch)
			{
				bSingleBranchSelected = false;
				SelectedChangeset.Reset();
				break;
			}
		}
	}
	const bool bSingleSelection = (SelectedChangesets.Num() == 1);
	const bool bDoubleSelection = (SelectedChangesets.Num() == 2);
	const bool bSingleNotCurrent = bSingleSelection && (SelectedChangeset->ChangesetId != CurrentChangesetId);

	static const FText SelectASingleChangesetTooltip(LOCTEXT("SelectASingleChangesetTooltip", "Select a single changeset."));
	static const FText SelectADifferentChangesetTooltip(LOCTEXT("SelectADifferentChangesetTooltip", "Select a changeset that is not the current one."));
	static const FText SelectASingleBranchTooltip(LOCTEXT("SelectASingleBranchTooltip", "Select changesets from a single branch."));

	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName = "PlasticSourceControl.ChangesetsContextMenu";
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	// Build up the menu
	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");

	Section.AddMenuEntry(
		"DiffChangeset",
		bSingleSelection ? FText::Format(LOCTEXT("DiffChangesetDynamic", "Diff changeset {0}"), FText::AsNumber(SelectedChangeset->ChangesetId)) : LOCTEXT("DiffChangeset", "Diff changeset"),
		bSingleSelection ? LOCTEXT("DiffChangesetTooltip", "Launch the Desktop application diff window showing changes in this changeset.") : SelectASingleChangesetTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnDiffChangesetClicked, SelectedChangeset),
			FCanExecuteAction::CreateLambda([bSingleSelection]() { return bSingleSelection; })
		)
	);

	Section.AddMenuEntry(
		"DiffChangesets",
		LOCTEXT("DiffChangesets", "Diff selected changesets"),
		bDoubleSelection ? LOCTEXT("DiffChangesetsTooltip", "Launch the Desktop application diff window showing changes between the two selected changesets.") : LOCTEXT("DoubleSelection", "Select a couple of changesets."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnDiffChangesetsClicked, SelectedChangesets),
			FCanExecuteAction::CreateLambda([bDoubleSelection]() { return bDoubleSelection; })
		)
	);

	Section.AddSeparator("PlasticSeparator1");

	Section.AddMenuEntry(
		"DiffBranch",
		bSingleBranchSelected ? FText::Format(LOCTEXT("DiffBranchDynamic", "Diff branch {0}"), FText::FromString(SelectedChangeset->Branch)) : LOCTEXT("DiffBranch", "Diff branch"),
		bSingleBranchSelected ? LOCTEXT("DiffBranchTooltip", "Launch the Desktop application diff window showing all changes in the selected branch.") : SelectASingleBranchTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnDiffBranchClicked, SelectedChangeset),
			FCanExecuteAction::CreateLambda([bSingleBranchSelected]() { return bSingleBranchSelected; })
		)
	);

	Section.AddSeparator("PlasticSeparator2");

	Section.AddMenuEntry(
		"SwitchToBranch",
		bSingleBranchSelected ? FText::Format(LOCTEXT("SwitchToBranchDynamic", "Switch workspace to the branch {0}"), FText::FromString(SelectedChangeset->Branch)) : LOCTEXT("SwitchToBranch", "Switch workspace to this branch"),
		bSingleBranchSelected ? LOCTEXT("SwitchToBranchTooltip", "Switch the workspace to the head of the branch with this changeset.") : SelectASingleBranchTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSwitchToBranchClicked, SelectedChangeset),
			FCanExecuteAction::CreateLambda([bSingleBranchSelected]() { return bSingleBranchSelected; })
		)
	);

	Section.AddMenuEntry(
		"SwitchToChangeset",
		bSingleSelection ? FText::Format(LOCTEXT("SwitchToChangesetDynamic", "Switch workspace to this changeset {0}"), FText::AsNumber(SelectedChangeset->ChangesetId)) : LOCTEXT("SwitchToChangeset", "Switch workspace to this changeset"),
		bSingleNotCurrent ? LOCTEXT("SwitchToChangesetsTooltip", "Switch the workspace to the specific changeset instead of a branch.\nSome information related to smart locks and to incoming changes won't be available.") :
		bSingleSelection ? SelectADifferentChangesetTooltip : SelectASingleChangesetTooltip,
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetClicked, SelectedChangeset),
			FCanExecuteAction::CreateLambda([bSingleNotCurrent]() { return bSingleNotCurrent; })
		)
	);

	// TODO: "Create branch from this changeset..." like in the Desktop application!

	return ToolMenus->GenerateWidget(Menu);
}

// Inspired by Engine\Source\Editor\SourceControlWindowExtender\Private\SourceControlWindowExtenderModule.cpp FSourceControlWindowExtenderModule::GetAssetsFromFilenames()
static void GetAssetsFromFilenames(const TArray<FString>& Filenames, TArray<FAssetData>& OutNonActorAssets, TArray<FAssetData>& OutCurrentWorldLoadedActors, TArray<FAssetData>& OutCurrentWorldUnloadedActors)
{
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();

	for (const FString& Filename : Filenames)
	{
		TArray<FAssetData> OutAssets;
		if (SourceControlHelpers::GetAssetData(Filename, OutAssets) && OutAssets.Num() == 1)
		{
			const FAssetData& AssetData = OutAssets[0];
			if (TSubclassOf<AActor> ActorClass = AssetData.GetClass())
			{
				if (CurrentWorld && AssetData.GetObjectPathString().StartsWith(CurrentWorld->GetPathName()))
				{
					if (AssetData.IsAssetLoaded())
					{
						OutCurrentWorldLoadedActors.Add(AssetData);
					}
					else
					{
						OutCurrentWorldUnloadedActors.Add(AssetData);
					}
				}
				else
				{
					TArray<FAssetData> OutWorldAsset;
					FString AssetPathName = AssetData.ToSoftObjectPath().GetLongPackageName();
					if (SourceControlHelpers::GetAssetDataFromPackage(AssetPathName, OutWorldAsset) && OutWorldAsset.Num() == 1)
					{
						OutNonActorAssets.Add(OutWorldAsset[0]);
					}
				}
			}
			else
			{
				OutNonActorAssets.Add(AssetData);
			}
		}
	}
}

static FString ConvertRelativePathToFull(const FPlasticSourceControlStatePtr& InSelectedFile)
{
	static const FString& WorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();
	return FPaths::Combine(WorkspaceRoot, InSelectedFile->LocalFilename);
}

static TArray<FString> ConvertRelativePathToFull(const TArray<FPlasticSourceControlStateRef>& InSelectedFiles)
{
	TArray<FString> AbsolutePaths;
	AbsolutePaths.Reserve(InSelectedFiles.Num());
	for (const FPlasticSourceControlStateRef& InSelectedFile : InSelectedFiles)
	{
		AbsolutePaths.Add(ConvertRelativePathToFull(InSelectedFile));
	}
	return AbsolutePaths;
}

TSharedPtr<SWidget> SPlasticSourceControlChangesetsWidget::OnOpenFileContextMenu()
{
	const TArray<FPlasticSourceControlStateRef> SelectedFiles = FilesListView->GetSelectedItems();
	if (SelectedFiles.Num() == 0)
	{
		return nullptr;
	}
	const FPlasticSourceControlStateRef SelectedFile = SelectedFiles[0];
	const bool bSingleSelection = (SelectedFiles.Num() == 1);

	static const FText SelectASingleFileTooltip(LOCTEXT("SelectASingleFileTooltip", "Select a single file."));

	// Make sure to only handle files, not directories, since we can't focus, diff or show their history in the Editor
	int32 DotIndex;
	if (!SelectedFile->LocalFilename.FindChar(TEXT('.'), DotIndex))
	{
		return nullptr;
	}

	// Note: none of the logic to populate the context menu cannot be used in UE5.0
	TArray<FAssetData> SelectedAssets;
	TArray<FAssetData> CurrentWorldLoadedActors;
	TArray<FAssetData> CurrentWorldUnloadedActors;
	GetAssetsFromFilenames(ConvertRelativePathToFull(SelectedFiles), SelectedAssets, CurrentWorldLoadedActors, CurrentWorldUnloadedActors);

	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName = "PlasticSourceControl.FilesContextMenu";
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	// Build up the menu
	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");

	// Only show the "Diff Against Previous" option if the selected file is "Changed" or "Moved" (not Added nor Deleted)
	if ((SelectedFile->WorkspaceState == EWorkspaceState::CheckedOutChanged) || (SelectedFile->WorkspaceState == EWorkspaceState::Moved)) // NOLINT(readability/braces)
	{
		Section.AddMenuEntry(
			"DiffAgainstPrevious",
			NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstPrev", "Diff Against Previous Revision"),
			bSingleSelection ? NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstPrevTooltip", "See changes between this revision and the previous one.") : SelectASingleFileTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnDiffRevisionClicked, SelectedFile),
				FCanExecuteAction::CreateLambda([bSingleSelection]() { return bSingleSelection; })
			)
		);
	}

	// Only show the "Diff Against Workspace" option if the selected asset is found in the workspace
	if ((SelectedAssets.Num() > 0) || (CurrentWorldLoadedActors.Num() > 0) || (CurrentWorldUnloadedActors.Num() > 0)) // NOLINT(readability/braces)
	{
		Section.AddMenuEntry(
			"DiffAgainstWorkspace",
			NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstWorkspace", "Diff Against Workspace File"),
			bSingleSelection ? NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstWorkspaceTooltip", "See changes between this revision and your version of the asset.") : SelectASingleFileTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnDiffAgainstWorkspaceClicked, SelectedFile),
				FCanExecuteAction::CreateLambda([bSingleSelection]() { return bSingleSelection; })
			)
		);
	}

	if (SelectedFile->History.Num() > 0)
	{
		Section.AddMenuEntry(
			"SaveRevision",
			LOCTEXT("SaveRevision", "Save this revision as..."),
			bSingleSelection ? LOCTEXT("SaveRevisionTooltip", "Save the selected revision of the file.") : SelectASingleFileTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSaveRevisionClicked, SelectedFile),
				FCanExecuteAction::CreateLambda([bSingleSelection]() { return bSingleSelection; })
			)
		);
	}

	// Note: this is a simplified heuristic, we might want to check that all files have a revision...
	if (SelectedFiles[0]->History.Num() > 0)
	{
		Section.AddSeparator("PlasticSeparator0");

		Section.AddMenuEntry(
			"RevertToRevision",
			LOCTEXT("RevertToRevision", "Revert files to this revision"),
			LOCTEXT("RevertToRevisionTooltip", "Revert these files to this revision, undoing any other changes done afterward."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnRevertToRevisionClicked, SelectedFiles),
				FCanExecuteAction()
			)
		);
	}

	// Only show the "Diff Against Workspace" option if the selected asset is found in the workspace
	// Note: as for now cm history does only work for assets found in the workspace, not if they were deleted
	if ((SelectedAssets.Num() > 0) || (CurrentWorldLoadedActors.Num() > 0) || (CurrentWorldUnloadedActors.Num() > 0))
	{
		Section.AddSeparator("PlasticSeparator1");

		Section.AddMenuEntry(
			"SCCHistory",
			LOCTEXT("SCCHistory", "History"),
			LOCTEXT("SCCHistoryTooltip", "Displays the history of the selected assets in revision control."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnShowHistoryClicked, SelectedFiles),
				FCanExecuteAction()
			)
		);
	}

	if (CurrentWorldLoadedActors.Num() > 0)
	{
		Section.AddSeparator("PlasticSeparator2");

		Section.AddMenuEntry(
			"SelectActors",
			LOCTEXT("SelectActors", "Select Actors"), LOCTEXT("SelectActors_Tooltip", "Select actors in the current level"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &SPlasticSourceControlChangesetsWidget::SelectActors, CurrentWorldLoadedActors)));
	}

	if (CurrentWorldLoadedActors.Num() > 0)
	{
		Section.AddMenuEntry(
			"FocusActors",
			LOCTEXT("FocusActors", "Focus Actors"), LOCTEXT("FocusActors_Tooltip", "Focus actors in the current level"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &SPlasticSourceControlChangesetsWidget::FocusActors, CurrentWorldLoadedActors)));
	}

	if (SelectedAssets.Num() > 0)
	{
		Section.AddSeparator("PlasticSeparator3");

		Section.AddMenuEntry(
			"BrowseToAssets",
			LOCTEXT("BrowseToAssets", "Browse to Assets"), LOCTEXT("BrowseToAssets_Tooltip", "Browse to Assets in Content Browser"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &SPlasticSourceControlChangesetsWidget::BrowseToAssets, SelectedAssets)));
	}

	return ToolMenus->GenerateWidget(Menu);
}

void SPlasticSourceControlChangesetsWidget::OnDiffChangesetClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset)
{
	if (InSelectedChangeset.IsValid())
	{
		PlasticSourceControlUtils::OpenDesktopApplicationForDiff(InSelectedChangeset->ChangesetId);
	}
}

void SPlasticSourceControlChangesetsWidget::OnDiffChangesetsClicked(TArray<FPlasticSourceControlChangesetRef> InSelectedChangesets)
{
	if (InSelectedChangesets.Num() == 2)
	{
		PlasticSourceControlUtils::OpenDesktopApplicationForDiff(InSelectedChangesets[0]->ChangesetId, InSelectedChangesets[1]->ChangesetId);
	}
}

void SPlasticSourceControlChangesetsWidget::OnDiffBranchClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset)
{
	if (InSelectedChangeset.IsValid())
	{
		PlasticSourceControlUtils::OpenDesktopApplicationForDiff(InSelectedChangeset->Branch);
	}
}

void SPlasticSourceControlChangesetsWidget::OnSwitchToBranchClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset)
{
	if (!InSelectedChangeset.IsValid())
		return;

	if (!Notification.IsInProgress())
	{
		// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them. Saving and checking out these assets will make the switch to the branch fail.
		PackageUtils::SaveDirtyPackages();

		// Find and Unlink all loaded packages in Content directory to allow to update them
		PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

		// Launch a custom "Switch" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToBranchOperation = ISourceControlOperation::Create<FPlasticSwitch>();
		SwitchToBranchOperation->BranchName = InSelectedChangeset->Branch;
		const ECommandResult::Type Result = Provider.Execute(SwitchToBranchOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSwitchToBranchOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(SwitchToBranchOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(SwitchToBranchOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset)
{
	const FText SwitchConfirmation = FText::Format(LOCTEXT("SwitchToChangesetDialog", "Are you sure you want to switch the workspace to the changeset {0} instead of a branch?\nSome information related to smart locks and to incoming changes won't be available."),
		FText::AsNumber(InSelectedChangeset->ChangesetId));
	const EAppReturnType::Type Choice = FMessageDialog::Open(
		EAppMsgCategory::Info,
		EAppMsgType::YesNo, SwitchConfirmation
		, LOCTEXT("SwitchToChangesetTitle", "Switch to changeset?")
	);
	if (Choice == EAppReturnType::Yes)
	{
		if (!Notification.IsInProgress())
		{
			// Launch a custom "Switch" operation

			// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them. Saving and checking out these assets will make the switch to the branch fail.
			PackageUtils::SaveDirtyPackages();

			// Find and Unlink all loaded packages in Content directory to allow to update them
			PackageUtils::UnlinkPackages(PackageUtils::ListAllPackages());

			FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
			if (!Provider.IsPartialWorkspace())
			{
				TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToChangesetOperation = ISourceControlOperation::Create<FPlasticSwitch>();
				SwitchToChangesetOperation->ChangesetId = InSelectedChangeset->ChangesetId;
				const ECommandResult::Type Result = Provider.Execute(SwitchToChangesetOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetOperationComplete));
				if (Result == ECommandResult::Succeeded)
				{
					// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
					Notification.DisplayInProgress(SwitchToChangesetOperation->GetInProgressString());
					StartRefreshStatus();
				}
				else
				{
					// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
					FNotification::DisplayFailure(SwitchToChangesetOperation.Get());
				}
			}
			else
			{
				TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> UpdateToChangesetOperation = ISourceControlOperation::Create<FPlasticSyncAll>();
				UpdateToChangesetOperation->SetRevision(FString::Printf(TEXT("%d"), InSelectedChangeset->ChangesetId));
				const ECommandResult::Type Result = Provider.Execute(UpdateToChangesetOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetOperationComplete));
				if (Result == ECommandResult::Succeeded)
				{
					// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
					Notification.DisplayInProgress(UpdateToChangesetOperation->GetInProgressString());
					StartRefreshStatus();
				}
				else
				{
					// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
					FNotification::DisplayFailure(UpdateToChangesetOperation.Get());
				}
			}
		}
		else
		{
			FMessageLog SourceControlLog("SourceControl");
			SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
			SourceControlLog.Notify();
		}
	}
}

static UObject* FindAssetInPackage(const FString& InLocalFilename, UPackage* InAssetPackage)
{
	UObject* AssetObject = nullptr;

	if (InAssetPackage)
	{
		const FString AssetName = FPaths::GetBaseFilename(InLocalFilename);

		AssetObject = FindObject<UObject>(InAssetPackage, *AssetName);

		// Recovery for package names that don't match
		if (!AssetObject)
		{
			AssetObject = InAssetPackage->FindAssetInPackage();
		}
	}

	return AssetObject;
}

static UPackage* LoadPackage(const FPlasticSourceControlStateRef& InSelectedFile)
{
	UPackage* AssetPackage = nullptr;

	const FString AbsolutePath = ConvertRelativePathToFull(InSelectedFile);
	FString AssetPackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(AbsolutePath, AssetPackageName))
	{
		AssetPackage = FindObject<UPackage>(nullptr, *AssetPackageName);
		if (!AssetPackage)
		{
			AssetPackage = LoadPackage(nullptr, *AssetPackageName, LOAD_None);
		}
	}

	return AssetPackage;
}

static UObject* GetAssetRevisionObject(FPlasticSourceControlRevisionRef& InRevision, FRevisionInfo& OutSelectedRevisionInfo)
{
	// try and load the temporary package
	UPackage* AssetPackage = DiffUtils::LoadPackageForDiff(InRevision);

	// grab the asset from the package - we assume asset name matches file name
	UObject* AssetObject = FindAssetInPackage(InRevision->Filename, AssetPackage);

	// fill out the revision info
	OutSelectedRevisionInfo.Revision = InRevision->Revision;
	OutSelectedRevisionInfo.Changelist = InRevision->ChangesetNumber;
	OutSelectedRevisionInfo.Date = InRevision->Date;

	return AssetObject;
}

static UObject* GetAssetRevisionObject(const FPlasticSourceControlStateRef& InSelectedFile, FRevisionInfo& OutSelectedRevisionInfo)
{
	FPlasticSourceControlRevisionRef SelectedRevision = InSelectedFile->History[0];

	// try and load the temporary package
	UPackage* AssetPackage = DiffUtils::LoadPackageForDiff(SelectedRevision);

	// grab the asset from the package
	UObject* AssetObject = FindAssetInPackage(InSelectedFile->LocalFilename, AssetPackage);

	// fill out the revision info
	OutSelectedRevisionInfo.Revision = SelectedRevision->Revision;
	OutSelectedRevisionInfo.Changelist = SelectedRevision->ChangesetNumber;
	OutSelectedRevisionInfo.Date = SelectedRevision->Date;

	return AssetObject;
}

static UObject* GetAssetWorkspaceObject(const FPlasticSourceControlStateRef& InSelectedFile)
{
	// need a package to find the asset in
	UPackage* AssetPackage = LoadPackage(InSelectedFile);

	// grab the asset from the package
	return FindAssetInPackage(InSelectedFile->LocalFilename, AssetPackage);
}

void SPlasticSourceControlChangesetsWidget::OnLocateFileClicked(FPlasticSourceControlStateRef InSelectedFile)
{
	// Behavior of the View Changes window: double click to focus on the file in the content browser or in the current level
	ISourceControlWindowsModule::Get().OnChangelistFileDoubleClicked().Broadcast(ConvertRelativePathToFull(InSelectedFile));
}

// Inspired by Engine\Source\Editor\SourceControlWindows\Private\SSourceControlHistoryWidget.cpp OnDiffAgainstPreviousRev()
void SPlasticSourceControlChangesetsWidget::OnDiffRevisionClicked(FPlasticSourceControlStateRef InSelectedFile)
{
	const FString AbsolutePath = ConvertRelativePathToFull(InSelectedFile);

	// Query for the file history for the provided packages
	// Note: this operation currently doesn't work for assets already removed from the workspace, as a limitation of "cm history"
	TArray<FString> PackageFilenames = TArray<FString>({ AbsolutePath });
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	if (Provider.Execute(UpdateStatusOperation, PackageFilenames))
	{
		// grab details on this file's state in source control (history, etc.)
		FPlasticSourceControlStateRef FileSourceControlState = Provider.GetStateInternal(AbsolutePath);

		if (FileSourceControlState->GetHistorySize() > 0)
		{
			// lookup the specific revision we want
			int32 SelectedRevisionIndex = ISourceControlState::INVALID_REVISION;
			{
				FPlasticSourceControlRevisionRef SelectedRevision = InSelectedFile->History[0];
				for (int32 i = 0; i < FileSourceControlState->History.Num(); i++)
				{
					FPlasticSourceControlRevisionRef Revision = FileSourceControlState->History[i];
					if (Revision->ChangesetNumber == SelectedRevision->ChangesetNumber)
					{
						SelectedRevisionIndex = i;
						break;
					}
				}
			}

			// History is starting from the latest revision at index 0, going upward for older/previous revisions
			if ((SelectedRevisionIndex != ISourceControlState::INVALID_REVISION) && (SelectedRevisionIndex < FileSourceControlState->History.Num() - 1))
			{
				const int32 PreviousRevisionIndex = SelectedRevisionIndex + 1;

				FRevisionInfo SelectedRevisionInfo;
				FPlasticSourceControlRevisionRef SelectedRevision = FileSourceControlState->History[SelectedRevisionIndex];
				UObject* SelectedAsset = GetAssetRevisionObject(SelectedRevision, SelectedRevisionInfo);

				FRevisionInfo PreviousRevisionInfo;
				FPlasticSourceControlRevisionRef PreviousRevision = FileSourceControlState->History[PreviousRevisionIndex];
				UObject* PreviousAsset = GetAssetRevisionObject(PreviousRevision, PreviousRevisionInfo);

				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
				AssetToolsModule.Get().DiffAssets(PreviousAsset, SelectedAsset, PreviousRevisionInfo, SelectedRevisionInfo);
			}
		}
	}
}

void SPlasticSourceControlChangesetsWidget::OnDiffAgainstWorkspaceClicked(FPlasticSourceControlStateRef InSelectedFile)
{
	if (InSelectedFile->History.Num() == 0)
	{
		return;
	}

	// grab the selected revision
	FRevisionInfo SelectedRevisionInfo;
	UObject* SelectedAsset = GetAssetRevisionObject(InSelectedFile, SelectedRevisionInfo);

	// we want the current working version of this asset
	FRevisionInfo CurrentRevisionInfo; // no revision info (empty string signify the current working version)
	UObject* CurrentAsset = GetAssetWorkspaceObject(InSelectedFile);

	// open the diff tool
	if (SelectedAsset && CurrentAsset)
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		AssetToolsModule.Get().DiffAssets(SelectedAsset, CurrentAsset, SelectedRevisionInfo, CurrentRevisionInfo);
	}
}

// Inspired by Engine\Source\Editor\UnrealEd\Private\FileHelpers.cpp FileDialogHelpers::SaveFile()
static bool SaveFile(const FString& Title, const FString& FileTypes, FString& InOutLastPath, const FString& DefaultFile, FString& OutFilename)
{
	bool bFileChosen = false;
	OutFilename = FString();

	TArray<FString> OutFilenames;
	bFileChosen = FDesktopPlatformModule::Get()->SaveFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		Title,
		InOutLastPath,
		DefaultFile,
		FileTypes,
		EFileDialogFlags::None,
		OutFilenames
	);

	bFileChosen = (OutFilenames.Num() > 0);

	if (bFileChosen)
	{
		// User successfully chose a file; remember the path for the next time the dialog opens.
		InOutLastPath = OutFilenames[0];
		OutFilename = OutFilenames[0];
	}

	return bFileChosen;
}

void SPlasticSourceControlChangesetsWidget::OnSaveRevisionClicked(FPlasticSourceControlStateRef InSelectedFile)
{
	check(InSelectedFile->History.Num() > 0);

	FPlasticSourceControlRevisionRef SelectedRevision = InSelectedFile->History[0];

	// Filter files based on the actual extension of the asset
	const FString Extension = FPaths::GetExtension(InSelectedFile->LocalFilename);
	const FString Filter = FString::Printf(TEXT("Assets (*.%s)|*.%s"), *Extension, *Extension);

	// Customize the filename with the revision number
	const FString BaseFilename = FPaths::GetBaseFilename(InSelectedFile->LocalFilename);
	const FString ProposedFilename = FString::Printf(TEXT("%s_cs%d.%s"), *BaseFilename, SelectedRevision->ChangesetNumber, *Extension);

	FString Filename;
	FString LastDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::UNR);
	const bool bFileChosen = SaveFile(LOCTEXT("SaveRevisionDialogTitle", "Save Revision").ToString(), Filter, LastDirectory, ProposedFilename, Filename);
	if (bFileChosen)
	{
		FEditorDirectories::Get().SetLastDirectory(ELastDirectory::UNR, LastDirectory);

		FString AbsolutePath = FPaths::ConvertRelativePathToFull(Filename);

		// save the selected revision
		if (SelectedRevision->Get(AbsolutePath))
		{
			UE_LOGF(LogSourceControl, Log, "Revision saved to '%ls'", *AbsolutePath);
		}
	}
}

void SPlasticSourceControlChangesetsWidget::OnRevertToRevisionClicked(TArray<FPlasticSourceControlStateRef> InSelectedFiles)
{
	check(InSelectedFiles.Num() > 0);
	check(InSelectedFiles[0]->History.Num() > 0);

	if (!Notification.IsInProgress())
	{
		// Warn the user about any unsaved assets (risk of losing work) but don't enforce saving them.
		PackageUtils::SaveDirtyPackages();

		const TArray<FString> Files = ConvertRelativePathToFull(InSelectedFiles);

		//  Unlink the selected packages to allow to revert them all
		PackageUtils::UnlinkPackages(Files);

		// Launch a custom "RevertToRevision" operation
		FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
		TSharedRef<FPlasticRevertToRevision, ESPMode::ThreadSafe> RevertToRevisionOperation = ISourceControlOperation::Create<FPlasticRevertToRevision>();
		FPlasticSourceControlRevisionRef SelectedRevision = InSelectedFiles[0]->History[0];
		RevertToRevisionOperation->ChangesetId = SelectedRevision->ChangesetNumber;
		const ECommandResult::Type Result = Provider.Execute(RevertToRevisionOperation, Files, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnRevertToRevisionOperationComplete));
		if (Result == ECommandResult::Succeeded)
		{
			// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
			Notification.DisplayInProgress(RevertToRevisionOperation->GetInProgressString());
			StartRefreshStatus();
		}
		else
		{
			// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
			FNotification::DisplayFailure(RevertToRevisionOperation.Get());
		}
	}
	else
	{
		FMessageLog SourceControlLog("SourceControl");
		SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
		SourceControlLog.Notify();
	}
}

void SPlasticSourceControlChangesetsWidget::OnShowHistoryClicked(TArray<FPlasticSourceControlStateRef> InSelectedFiles)
{
	// Note: it's not worth trying to support selection of multiple files
	FSourceControlWindows::DisplayRevisionHistory(ConvertRelativePathToFull(InSelectedFiles));
}

// Inspired by Engine\Source\Editor\SourceControlWindowExtender\Private\SourceControlWindowExtenderModule.cpp FSourceControlWindowExtenderModule::()
// Note: all these are only supported for versions after UE5.0
// Note: all these are ready for multiple selection even though we don't support it yet
void SPlasticSourceControlChangesetsWidget::SelectActors(const TArray<FAssetData> InActorsToSelect)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectActorsFromChangelist", "Select Actor(s)"));
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	check(CurrentWorld);

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bNotify = false;
	const bool bDeselectBSPSurfs = true;
	GEditor->SelectNone(bNotify, bDeselectBSPSurfs);

	for (const FAssetData& ActorToSelect : InActorsToSelect)
	{
		if (AActor* Actor = Cast<AActor>(ActorToSelect.FastGetAsset()))
		{
			const bool bSelected = true;
			GEditor->SelectActor(Actor, bSelected, bNotify);
		}
	}

	bNotify = true;
	GEditor->GetSelectedActors()->EndBatchSelectOperation(bNotify);
}

void SPlasticSourceControlChangesetsWidget::FocusActors(const TArray<FAssetData> InActorToFocus)
{
	FBox FocusBounds(EForceInit::ForceInit);
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	check(CurrentWorld);
	for (const FAssetData& ActorToFocus : InActorToFocus)
	{
		if (TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorToFocus))
		{
			const FBox EditorBounds = ActorDesc->GetEditorBounds();
			if (EditorBounds.IsValid)
			{
				FocusBounds += EditorBounds;
			}
		}
	}

	if (FocusBounds.IsValid)
	{
		const bool bActiveViewportOnly = true;
		const float TimeInSeconds = 0.5f;
		GEditor->MoveViewportCamerasToBox(FocusBounds, bActiveViewportOnly, TimeInSeconds);
	}
}

void SPlasticSourceControlChangesetsWidget::BrowseToAssets(const TArray<FAssetData> InAssets)
{
	GEditor->SyncBrowserToObjects(const_cast<TArray<FAssetData>&>(InAssets)); // Note: const cast for UE5.2
}

void SPlasticSourceControlChangesetsWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	// Detect transitions of the source control being available/unavailable. Ex: When the user changes the source control in UI, the provider gets selected,
	// but it is not connected/available until the user accepts the settings. The source control doesn't have callback for availability and we want to refresh everything
	// once it gets available.
	if (ISourceControlModule::Get().IsEnabled() && !bSourceControlAvailable && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		bSourceControlAvailable = true;
		bShouldRefresh = true;
	}

	// Auto refresh at regular intervals
	const double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - LastRefreshTime > (10 * 60))
	{
		bShouldRefresh = true;
	}

	if (bShouldRefresh)
	{
		RequestChangesetsRefresh();
		LastRefreshTime = CurrentTime;
		bShouldRefresh = false;
	}

	if (bIsRefreshing)
	{
		TickRefreshStatus(InDeltaTime);
	}
}

void SPlasticSourceControlChangesetsWidget::StartRefreshStatus()
{
	if (!bIsRefreshing)
	{
		bIsRefreshing = true;
		RefreshStatusStartSecs = FPlatformTime::Seconds();
	}
}

void SPlasticSourceControlChangesetsWidget::TickRefreshStatus(double InDeltaTime)
{
	const int32 RefreshStatusTimeElapsed = static_cast<int32>(FPlatformTime::Seconds() - RefreshStatusStartSecs);
	RefreshStatus = FText::Format(LOCTEXT("PlasticSourceControl_RefreshChangesets", "Refreshing changesets... ({0} s)"), FText::AsNumber(RefreshStatusTimeElapsed));
}

void SPlasticSourceControlChangesetsWidget::EndRefreshStatus()
{
	bIsRefreshing = false;
	RefreshStatus = FText::GetEmpty();
}

void SPlasticSourceControlChangesetsWidget::RequestChangesetsRefresh()
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	StartRefreshStatus();

	TSharedRef<FPlasticGetChangesets, ESPMode::ThreadSafe> GetChangesetsOperation = ISourceControlOperation::Create<FPlasticGetChangesets>();
	if (FromDateInDays > -1)
	{
		GetChangesetsOperation->FromDate = FDateTime::Now() - FTimespan::FromDays(FromDateInDays);
	}

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	Provider.Execute(GetChangesetsOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnGetChangesetsOperationComplete));
}

void SPlasticSourceControlChangesetsWidget::RequestGetChangesetFiles(const FPlasticSourceControlChangesetPtr& InSelectedChangeset)
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	StartRefreshStatus();

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	TSharedRef<FPlasticGetChangesetFiles, ESPMode::ThreadSafe> GetChangesetFilesOperation = ISourceControlOperation::Create<FPlasticGetChangesetFiles>();
	GetChangesetFilesOperation->Changeset = InSelectedChangeset;
	Provider.Execute(GetChangesetFilesOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlChangesetsWidget::OnGetChangesetFilesOperationComplete));
}

void SPlasticSourceControlChangesetsWidget::OnGetChangesetsOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TSharedRef<FPlasticGetChangesets, ESPMode::ThreadSafe> GetChangesetsOperation = StaticCastSharedRef<FPlasticGetChangesets>(InOperation);
	SourceControlChangesets = MoveTemp(GetChangesetsOperation->Changesets);

	CurrentChangesetId = FPlasticSourceControlModule::Get().GetProvider().GetChangesetNumber();

	EndRefreshStatus();
	OnChangesetsRefreshUI();
}

void SPlasticSourceControlChangesetsWidget::OnGetChangesetFilesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TSharedRef<FPlasticGetChangesetFiles, ESPMode::ThreadSafe> GetChangesetFilesOperation = StaticCastSharedRef<FPlasticGetChangesetFiles>(InOperation);
	GetChangesetFilesOperation->Changeset->Files = MoveTemp(GetChangesetFilesOperation->Files);

	EndRefreshStatus();
	OnFilesRefreshUI();
}

void SPlasticSourceControlChangesetsWidget::OnSwitchToBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::OnSwitchToBranchOperationComplete);

	// Reload packages that where updated by the SwitchToBranch operation (and the current map if needed)
	TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToBranchOperation = StaticCastSharedRef<FPlasticSwitch>(InOperation);
	PackageUtils::ReloadPackages(SwitchToBranchOperation->UpdatedFiles);

	// Ask for a full refresh of the list of changesets (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::OnSwitchToChangesetOperationComplete);

	// Reload packages that where updated by the SwitchToChangeset operation (and the current map if needed)
	if (!FPlasticSourceControlModule::Get().GetProvider().IsPartialWorkspace())
	{
		TSharedRef<FPlasticSwitch, ESPMode::ThreadSafe> SwitchToChangesetOperation = StaticCastSharedRef<FPlasticSwitch>(InOperation);
		PackageUtils::ReloadPackages(SwitchToChangesetOperation->UpdatedFiles);
	}
	else
	{
		TSharedRef<FPlasticSyncAll, ESPMode::ThreadSafe> UpdateToChangesetOperation = StaticCastSharedRef<FPlasticSyncAll>(InOperation);
		PackageUtils::ReloadPackages(UpdateToChangesetOperation->UpdatedFiles);
	}

	// Ask for a full refresh of the list of changesets (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlChangesetsWidget::OnRevertToRevisionOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlChangesetsWidget::OnRevertToRevisionOperationComplete);

	// Reload packages that where updated by the RevertToRevision operation (and the current map if needed)
	TSharedRef<FPlasticRevertToRevision, ESPMode::ThreadSafe> RevertToRevisionOperation = StaticCastSharedRef<FPlasticRevertToRevision>(InOperation);
	PackageUtils::ReloadPackages(RevertToRevisionOperation->UpdatedFiles);

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlChangesetsWidget::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	bSourceControlAvailable = NewProvider.IsAvailable(); // Check if it is connected.
	bShouldRefresh = true;

	if (&NewProvider != &OldProvider)
	{
		ChangesetRows.Reset();
		if (ChangesetsListView)
		{
			ChangesetsListView->RequestListRefresh();
		}

		FileRows.Reset();
		if (FilesListView)
		{
			FilesListView->RequestListRefresh();
		}
	}
}

void SPlasticSourceControlChangesetsWidget::HandleSourceControlStateChanged()
{
	bShouldRefresh = true;
	if (ChangesetsListView)
	{
		ChangesetsListView->RequestListRefresh();
	}
	if (FilesListView)
	{
		FilesListView->RequestListRefresh();
	}
}

// On changeset selected, show its list of files changed
void SPlasticSourceControlChangesetsWidget::OnSelectionChanged(FPlasticSourceControlChangesetPtr InSelectedChangeset, ESelectInfo::Type SelectInfo)
{
	SourceSelectedChangeset = InSelectedChangeset;

	if (InSelectedChangeset.IsValid() && SourceSelectedChangeset->Files.Num() == 0)
	{
		// Asynchronously get the list of files changed in the changeset
		RequestGetChangesetFiles(SourceSelectedChangeset);
	}
	else
	{
		// Just refresh the list of files
		OnFilesRefreshUI();
	}
}

void SPlasticSourceControlChangesetsWidget::OnItemDoubleClicked(FPlasticSourceControlChangesetRef InSelectedChangeset)
{
	OnDiffChangesetClicked(InSelectedChangeset);
}

FReply SPlasticSourceControlChangesetsWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F5)
	{
		// Pressing F5 refreshes the list of changesets
		bShouldRefresh = true;
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Pressing Enter open the diff for the selected file or the selected changeset (like a double click)
		if (ChangesetsListView && FilesListView)
		{
			const TArray<FPlasticSourceControlStateRef> SelectedFiles = FilesListView->GetSelectedItems();
			if (SelectedFiles.Num() == 1)
			{
				OnDiffRevisionClicked(SelectedFiles[0]);
			}
			else
			{
				const TArray<FPlasticSourceControlChangesetRef> SelectedChangesets = ChangesetsListView->GetSelectedItems();
				if (SelectedChangesets.Num() == 1)
				{
					OnDiffChangesetClicked(SelectedChangesets[0]);
				}
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE