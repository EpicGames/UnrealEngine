// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Notification.h"

#include "Misc/TextFilter.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"

class FPlasticSourceControlChangeset;
class FPlasticSourceControlState;
typedef TSharedRef<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetRef;
typedef TSharedPtr<class FPlasticSourceControlChangeset, ESPMode::ThreadSafe> FPlasticSourceControlChangesetPtr;
typedef TSharedRef<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStateRef;
typedef TSharedPtr<class FPlasticSourceControlState, ESPMode::ThreadSafe> FPlasticSourceControlStatePtr;

class SSearchBox;

// Widget displaying the list of Changesets in the tab window, see FPlasticSourceControlChangesetsWindow
class SPlasticSourceControlChangesetsWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPlasticSourceControlChangesetsWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TSharedRef<SWidget> CreateToolBar();
	TSharedRef<SWidget> CreateChangesetsListView();
	TSharedRef<SWidget> CreateFilesListView();

	TSharedRef<ITableRow> OnGenerateRow(FPlasticSourceControlChangesetRef InChangeset, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> OnGenerateRow(FPlasticSourceControlStateRef InChangeset, const TSharedRef<STableViewBase>& OwnerTable);
	void OnHiddenColumnsListChanged();

	void OnChangesetsSearchTextChanged(const FText& InFilterText);
	void OnFilesSearchTextChanged(const FText& InFilterText);
	void PopulateItemSearchStrings(const FPlasticSourceControlChangeset& InItem, TArray<FString>& OutStrings);
	void PopulateItemSearchStrings(const FPlasticSourceControlState& InItem, TArray<FString>& OutStrings);

	TSharedRef<SWidget> BuildFromDateDropDownMenu();
	void OnFromDateChanged(int32 InFromDateInDays);

	void OnChangesetsRefreshUI();
	void OnFilesRefreshUI();

	EColumnSortPriority::Type GetChangesetsColumnSortPriority(const FName InColumnId) const;
	EColumnSortMode::Type GetChangesetsColumnSortMode(const FName InColumnId) const;
	void OnChangesetsColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

	EColumnSortPriority::Type GetFilesColumnSortPriority(const FName InColumnId) const;
	EColumnSortMode::Type GetFilesColumnSortMode(const FName InColumnId) const;
	void OnFilesColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

	void SortChangesetsView();
	void SortFilesView();

	TSharedPtr<SWidget> OnOpenChangesetContextMenu();
	TSharedPtr<SWidget> OnOpenFileContextMenu();

	void OnDiffChangesetClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset);
	void OnDiffChangesetsClicked(TArray<FPlasticSourceControlChangesetRef> InSelectedChangesets);
	void OnDiffBranchClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset);
	void OnSwitchToBranchClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset);
	void OnSwitchToChangesetClicked(FPlasticSourceControlChangesetPtr InSelectedChangeset);
	void OnLocateFileClicked(FPlasticSourceControlStateRef InSelectedFile);
	void OnDiffRevisionClicked(FPlasticSourceControlStateRef InSelectedFile);
	void OnDiffAgainstWorkspaceClicked(FPlasticSourceControlStateRef InSelectedFile);
	void OnSaveRevisionClicked(FPlasticSourceControlStateRef InSelectedFile);
	void OnRevertToRevisionClicked(TArray<FPlasticSourceControlStateRef> InSelectedFiles);
	void OnShowHistoryClicked(TArray<FPlasticSourceControlStateRef> InSelectedFiles);

	void SelectActors(const TArray<FAssetData> ActorsToSelect);
	void FocusActors(const TArray<FAssetData> ActorToFocus);
	void BrowseToAssets(const TArray<FAssetData> Assets);

	void StartRefreshStatus();
	void TickRefreshStatus(double InDeltaTime);
	void EndRefreshStatus();

	void RequestChangesetsRefresh();
	void RequestGetChangesetFiles(const FPlasticSourceControlChangesetPtr& InSelectedChangeset);

	/** Source control callbacks */
	void OnGetChangesetsOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnGetChangesetFilesOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnSwitchToBranchOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnSwitchToChangesetOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnRevertToRevisionOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

	/** Delegate handler for when source control state changes */
	void HandleSourceControlStateChanged();

	void OnSelectionChanged(FPlasticSourceControlChangesetPtr InSelectedChangeset, ESelectInfo::Type SelectInfo);

	/** Double click to diff the selected changeset */
	void OnItemDoubleClicked(FPlasticSourceControlChangesetRef InChangeset);

	/** Interpret F5 and Enter keys */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TSharedPtr<SSearchBox> ChangesetsSearchBox;
	TSharedPtr<SSearchBox> FilesSearchBox;

	FName ChangesetsPrimarySortedColumn;
	FName ChangesetsSecondarySortedColumn;
	EColumnSortMode::Type ChangesetsPrimarySortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type ChangesetsSecondarySortMode = EColumnSortMode::None;

	FName FilesPrimarySortedColumn;
	FName FilesSecondarySortedColumn;
	EColumnSortMode::Type FilesPrimarySortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type FilesSecondarySortMode = EColumnSortMode::None;

	TArray<FName> ChangesetsHiddenColumnsList;

	bool bShouldRefresh = false;
	bool bSourceControlAvailable = false;

	FText RefreshStatus;
	bool bIsRefreshing = false;
	double RefreshStatusStartSecs;
	double LastRefreshTime = 0.0;

	int32 CurrentChangesetId;

	/** Ongoing notification for a long-running asynchronous source control operation, if any */
	FNotification Notification;

	TSharedPtr<SListView<FPlasticSourceControlChangesetRef>> ChangesetsListView;
	TSharedPtr<TTextFilter<const FPlasticSourceControlChangeset&>> ChangesetsSearchTextFilter;

	TMap<int32, FText> FromDateInDaysValues;
	int32 FromDateInDays = 30;

	TArray<FPlasticSourceControlChangesetRef> SourceControlChangesets; // Full list from source (filtered by date)
	TArray<FPlasticSourceControlChangesetRef> ChangesetRows; // Filtered list to display based on the search text filter

	TSharedPtr<SListView<FPlasticSourceControlStateRef>> FilesListView;
	TSharedPtr<TTextFilter<const FPlasticSourceControlState&>> FilesSearchTextFilter;

	FPlasticSourceControlChangesetPtr SourceSelectedChangeset; // Current selected changeset from source control if any, with full list of files
	TArray<FPlasticSourceControlStateRef> FileRows; // Filtered list to display based on the search text filter

	/** Delegate handle for the HandleSourceControlStateChanged function callback */
	FDelegateHandle SourceControlStateChangedDelegateHandle;
};
