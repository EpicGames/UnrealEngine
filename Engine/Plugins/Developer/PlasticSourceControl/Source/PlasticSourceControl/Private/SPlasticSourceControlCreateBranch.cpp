// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlCreateBranch.h"

#include "SPlasticSourceControlBranchesWidget.h"

#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

void SPlasticSourceControlCreateBranch::Construct(const FArguments& InArgs)
{
	BranchesWidget = InArgs._BranchesWidget;
	ParentWindow = InArgs._ParentWindow;
	ParentBranchName = InArgs._ParentBranchName;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("PlasticCreateBrancheDetails", "Create a new child branch from last changeset on branch {0}"), FText::FromString(ParentBranchName)))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("PlasticCreateBrancheNameTooltip", "Enter a name for the new branch to create"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlasticCreateBrancheNameLabel", "Branch name:"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(6.0f)
			[
				SAssignNew(BranchNameTextBox, SEditableTextBox)
				.HintText(LOCTEXT("PlasticCreateBrancheNameHint", "Name of the new branch"))
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					NewBranchName = InText.ToString();
				})
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
				{
					NewBranchName = InText.ToString();
					if (TextCommitType == ETextCommit::OnEnter)
					{
						CreateClicked();
					}
				})
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("PlasticCreateBrancheCommentTooltip", "Enter optional comments for the new branch"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlasticCreateBrancheCommentLabel", "Comments:"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(6.0f)
			[
				SNew(SBox)
				.MinDesiredHeight(120.0f)
				.WidthOverride(520.0f)
				[
					SNew(SMultiLineEditableTextBox)
					.AutoWrapText(true)
					.HintText(LOCTEXT("PlasticCreateBrancheCommentHing", "Comments for the new branch"))
					.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
					{
						NewBranchComment = InText.ToString();
					})
				]
			]
		]
		// Option to switch workspace to this branch
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bSwitchWorkspace ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(this, &SPlasticSourceControlCreateBranch::OnCheckedSwitchWorkspace)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PlasticSwitchWorkspace", "Switch workspace to this branch"))
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Create", "Create"))
				.IsEnabled(this, &SPlasticSourceControlCreateBranch::CanCreateBranch)
				.ToolTipText(this, &SPlasticSourceControlCreateBranch::CreateButtonTooltip)
				.OnClicked(this, &SPlasticSourceControlCreateBranch::CreateClicked)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SPlasticSourceControlCreateBranch::CancelClicked)
			]
		]
	];

	ParentWindow.Pin()->SetWidgetToFocusOnActivate(BranchNameTextBox);
}

inline void SPlasticSourceControlCreateBranch::OnCheckedSwitchWorkspace(ECheckBoxState InState)
{
	bSwitchWorkspace = (InState == ECheckBoxState::Checked);
}


bool SPlasticSourceControlCreateBranch::CanCreateBranch() const
{
	if (NewBranchName.IsEmpty())
	{
		return false;
	}

	return SPlasticSourceControlBranchesWidget::IsBranchNameValid(NewBranchName);
}

FText SPlasticSourceControlCreateBranch::CreateButtonTooltip() const
{
	if (NewBranchName.IsEmpty())
	{
		return LOCTEXT("CreateEmpty_Tooltip", "Enter a name for the new branch.");
	}

	if (!SPlasticSourceControlBranchesWidget::IsBranchNameValid(NewBranchName))
	{
		return LOCTEXT("CreateInvalid_Tooltip", "Branch name cannot contain any of the following characters: @#/:\"?'\\n\\r\\t");
	}

	return FText::Format(LOCTEXT("CreateBranch_Tooltip", "Create branch {0}."),
		FText::FromString(ParentBranchName + TEXT("/") + NewBranchName));
}

FReply SPlasticSourceControlCreateBranch::CreateClicked()
{
	if (TSharedPtr<SPlasticSourceControlBranchesWidget> Branches = BranchesWidget.Pin())
	{
		Branches->CreateBranch(ParentBranchName, NewBranchName, NewBranchComment, bSwitchWorkspace);
	}

	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlCreateBranch::CancelClicked()
{
	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlCreateBranch::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Pressing Escape returns as if the user clicked Cancel
		return CancelClicked();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
