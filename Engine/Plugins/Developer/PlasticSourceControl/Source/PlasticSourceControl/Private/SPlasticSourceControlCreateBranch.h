// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
class SEditableTextBox;
class SPlasticSourceControlBranchesWidget;
class SWindow;
enum class ECheckBoxState : uint8;

class SPlasticSourceControlCreateBranch : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlCreateBranch)
		: _BranchesWidget()
		, _ParentWindow()
		, _ParentBranchName()
	{}

		SLATE_ARGUMENT(TSharedPtr<SPlasticSourceControlBranchesWidget>, BranchesWidget)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(FString, ParentBranchName)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void OnCheckedSwitchWorkspace(ECheckBoxState InState);

	bool CanCreateBranch() const;
	FText CreateButtonTooltip() const;

	FReply CreateClicked();
	FReply CancelClicked();

	/** Interpret Escape as Cancel */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	FString ParentBranchName;
	FString NewBranchName;
	FString NewBranchComment;
	bool bSwitchWorkspace = true;

	TSharedPtr<SEditableTextBox> BranchNameTextBox;

	TWeakPtr<SPlasticSourceControlBranchesWidget> BranchesWidget;
	TWeakPtr<SWindow> ParentWindow;
};
