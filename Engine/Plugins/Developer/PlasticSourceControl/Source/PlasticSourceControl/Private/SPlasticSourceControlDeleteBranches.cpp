// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlDeleteBranches.h"

#include "SPlasticSourceControlBranchesWidget.h"

#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

void SPlasticSourceControlDeleteBranches::Construct(const FArguments& InArgs)
{
	BranchesWidget = InArgs._BranchesWidget;
	ParentWindow = InArgs._ParentWindow;
	BranchNames = InArgs._BranchNames;

	FString Branches;
	for (int32 i = 0; i < FMath::Min(BranchNames.Num(), 10); i++)
	{
		const FString& BranchName = BranchNames[i];
		Branches += BranchName + TEXT("\n");
		if (i == 9 && BranchNames.Num() > 10)
		{
			Branches += FString::Printf(TEXT("... and %d others."), BranchNames.Num() - 10);
		}
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(BranchNames.Num() == 1
					? LOCTEXT("PlasticDeleteBranchDetails", "You are about to delete 1 branch:")
					: FText::Format(LOCTEXT("PlasticDeleteBranchesDetails", "You are about to delete {0} branches:"), FText::AsNumber(BranchNames.Num())))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::FromString(Branches))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(DeleteButtonPtr, SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Delete", "Delete"))
				.OnClicked(this, &SPlasticSourceControlDeleteBranches::DeleteClicked)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SPlasticSourceControlDeleteBranches::CancelClicked)
			]
		]
	];

	ParentWindow.Pin()->SetWidgetToFocusOnActivate(DeleteButtonPtr);
}

FReply SPlasticSourceControlDeleteBranches::DeleteClicked()
{
	if (TSharedPtr<SPlasticSourceControlBranchesWidget> Branches = BranchesWidget.Pin())
	{
		Branches->DeleteBranches(BranchNames);
	}

	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlDeleteBranches::CancelClicked()
{
	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlDeleteBranches::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Pressing Escape returns as if the user clicked Cancel
		return CancelClicked();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
