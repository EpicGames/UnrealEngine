// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
class SButton;
class SPlasticSourceControlBranchesWidget;
class SWindow;

class SPlasticSourceControlDeleteBranches : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlDeleteBranches)
		: _BranchesWidget()
		, _ParentWindow()
		, _BranchNames()
	{}

		SLATE_ARGUMENT(TSharedPtr<SPlasticSourceControlBranchesWidget>, BranchesWidget)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TArray<FString>, BranchNames)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FReply DeleteClicked();
	FReply CancelClicked();

	/** Interpret Escape as Cancel */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TArray<FString> BranchNames;

	TWeakPtr<SPlasticSourceControlBranchesWidget> BranchesWidget;
	TWeakPtr<SWindow> ParentWindow;

	TSharedPtr<SButton> DeleteButtonPtr;
};
