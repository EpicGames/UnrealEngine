// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlLockRow.h"

#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlUtils.h"

#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlLockWindow"

FName PlasticSourceControlLocksListViewColumn::ItemId::Id() { return TEXT("ItemId"); }
FText PlasticSourceControlLocksListViewColumn::ItemId::GetDisplayText() { return LOCTEXT("Id_Column", "Item Id"); }
FText PlasticSourceControlLocksListViewColumn::ItemId::GetToolTipText() { return LOCTEXT("Id_Column_Tooltip", "Displays the Id of the locked Item"); }

FName PlasticSourceControlLocksListViewColumn::Path::Id() { return TEXT("Path"); }
FText PlasticSourceControlLocksListViewColumn::Path::GetDisplayText() { return LOCTEXT("Path_Column", "Item"); }
FText PlasticSourceControlLocksListViewColumn::Path::GetToolTipText() { return LOCTEXT("Path_Column_Tooltip", "Displays the item path"); }

FName PlasticSourceControlLocksListViewColumn::Status::Id() { return TEXT("Status"); }
FText PlasticSourceControlLocksListViewColumn::Status::GetDisplayText() { return LOCTEXT("Status_Column", "Status"); }
FText PlasticSourceControlLocksListViewColumn::Status::GetToolTipText() { return LOCTEXT("Status_Column_Tooltip", "Displays the lock status"); }

FName PlasticSourceControlLocksListViewColumn::Date::Id() { return TEXT("Date"); }
FText PlasticSourceControlLocksListViewColumn::Date::GetDisplayText() { return LOCTEXT("Date_Column", "Modification date"); }
FText PlasticSourceControlLocksListViewColumn::Date::GetToolTipText() { return LOCTEXT("Date_Column_Tooltip", "Displays the lock modification date"); }

FName PlasticSourceControlLocksListViewColumn::Owner::Id() { return TEXT("Owner"); }
FText PlasticSourceControlLocksListViewColumn::Owner::GetDisplayText() { return LOCTEXT("Owner_Column", "Owner"); }
FText PlasticSourceControlLocksListViewColumn::Owner::GetToolTipText() { return LOCTEXT("Owner_Column_Tooltip", "Displays the name of the owner of the lock"); }

FName PlasticSourceControlLocksListViewColumn::DestinationBranch::Id() { return TEXT("Destination Branch"); }
FText PlasticSourceControlLocksListViewColumn::DestinationBranch::GetDisplayText() { return LOCTEXT("DestinationBranch_Column", "Destination Branch"); }
FText PlasticSourceControlLocksListViewColumn::DestinationBranch::GetToolTipText() { return LOCTEXT("DestinationBranch_Column_Tooltip", "Displays the branch where the merge needs to happen in order to remove the lock"); }

FName PlasticSourceControlLocksListViewColumn::Branch::Id() { return TEXT("Branch"); }
FText PlasticSourceControlLocksListViewColumn::Branch::GetDisplayText() { return LOCTEXT("Branch_Column", "Branch"); }
FText PlasticSourceControlLocksListViewColumn::Branch::GetToolTipText() { return LOCTEXT("Branch_Column_Tooltip", "Displays the branch where the lock has been created"); }

FName PlasticSourceControlLocksListViewColumn::Workspace::Id() { return TEXT("Workspace"); }
FText PlasticSourceControlLocksListViewColumn::Workspace::GetDisplayText() { return LOCTEXT("Workspace_Column", "Workspace"); }
FText PlasticSourceControlLocksListViewColumn::Workspace::GetToolTipText() { return LOCTEXT("Workspace_Column_Tooltip", "Displays the workspace where the lock has been created"); }

void SPlasticSourceControlLockRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	LockToVisualize = InArgs._LockToVisualize.Get();
	HighlightText = InArgs._HighlightText;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> SPlasticSourceControlLockRow::GenerateWidgetForColumn(const FName& InColumnId)
{
	if (InColumnId == PlasticSourceControlLocksListViewColumn::ItemId::Id())
	{
		return SNew(STextBlock)
			.Text(FText::AsNumber(LockToVisualize->ItemId))
			.ToolTipText(FText::AsNumber(LockToVisualize->ItemId))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Path::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(LockToVisualize->Path))
			.ToolTipText(FText::FromString(LockToVisualize->Path))
			.Margin(FMargin(6.f, 1.f))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Status::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(LockToVisualize->Status))
			.ToolTipText(FText::FromString(LockToVisualize->Status))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Date::Id())
	{
		return SNew(STextBlock)
			.Text(FText::AsDateTime(LockToVisualize->Date))
			.ToolTipText(FText::AsDateTime(LockToVisualize->Date))
			.Margin(FMargin(6.f, 1.f));
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Owner::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(PlasticSourceControlUtils::UserNameToDisplayName(LockToVisualize->Owner)))
			.ToolTipText(FText::FromString(LockToVisualize->Owner))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(LockToVisualize->DestinationBranch))
			.ToolTipText(FText::FromString(LockToVisualize->DestinationBranch))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Branch::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(LockToVisualize->Branch))
			.ToolTipText(FText::FromString(LockToVisualize->Branch))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else if (InColumnId == PlasticSourceControlLocksListViewColumn::Workspace::Id())
	{
		return SNew(STextBlock)
			.Text(FText::FromString(LockToVisualize->Workspace))
			.ToolTipText(FText::FromString(LockToVisualize->Workspace))
			.Margin(FMargin(6.f, 1.f))
			.HighlightText(HighlightText);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

#undef LOCTEXT_NAMESPACE
