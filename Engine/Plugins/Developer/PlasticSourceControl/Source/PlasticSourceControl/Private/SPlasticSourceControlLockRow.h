// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;
typedef TSharedPtr<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockPtr;

class FPlasticSourceControlLock;

/** Lists the unique columns used in the list view displaying locks. */
namespace PlasticSourceControlLocksListViewColumn
{
	/** The lock ItemId column. */
	namespace ItemId // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Item Path column. */
	namespace Path // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Status column. */
	namespace Status // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Date column. */
	namespace Date // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Owner column. */
	namespace Owner // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Destination Branch column. */
	namespace DestinationBranch // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Branch Holder column. */
	namespace Branch // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};

	/** The lock Workspace column. */
	namespace Workspace // NOLINT(runtime/indentation_namespace)
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	};
} // namespace PlasticSourceControlLocksListViewColumn

class SPlasticSourceControlLockRow : public SMultiColumnTableRow<FPlasticSourceControlLockRef>
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlLockRow)
		: _LockToVisualize(nullptr)
		, _HighlightText()
	{
	}
		SLATE_ARGUMENT(FPlasticSourceControlLockPtr, LockToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:
	/**
	* Construct a row child widgets of the ListView.
	*
	* @param InArgs Parameters including the lock to visualize in this row.
	* @param InOwner The owning ListView.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

private:
	/** The lock that we are visualizing in this row. */
	FPlasticSourceControlLock* LockToVisualize;

	/** The search text to highlight if any */
	TAttribute<FText> HighlightText;
};
