// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlLocksWidget.h"

#include "PlasticSourceControlLock.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlProjectSettings.h"
#include "PlasticSourceControlUtils.h"
#include "SPlasticSourceControlLockRow.h"

#include "PackageUtils.h"

#include "ISourceControlModule.h"

#include "Logging/MessageLog.h"
#include "ToolMenus.h"
#include "ToolMenuContext.h"

#include "Misc/ComparisonUtility.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlLockWindow"

void SPlasticSourceControlLocksWidget::Construct(const FArguments& InArgs)
{
	ISourceControlModule::Get().RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SPlasticSourceControlLocksWidget::OnSourceControlProviderChanged));
	// register for any source control change to detect new local locks on check-out, and release of them on check-in
	SourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SPlasticSourceControlLocksWidget::HandleSourceControlStateChanged));

	WorkspaceSelector = FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector();

	const FString OrganizationName = FPlasticSourceControlModule::Get().GetProvider().GetCloudOrganization();

	SearchTextFilter = MakeShared<TTextFilter<const FPlasticSourceControlLock&>>(TTextFilter<const FPlasticSourceControlLock&>::FItemToStringArray::CreateSP(this, &SPlasticSourceControlLocksWidget::PopulateItemSearchStrings));
	SearchTextFilter->OnChanged().AddSP(this, &SPlasticSourceControlLocksWidget::OnRefreshUI);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot() // For the toolbar (Search box and Refresh button)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						CreateToolBar()
					]
					+SHorizontalBox::Slot()
					.MaxWidth(10.0f)
					[
						SNew(SSpacer)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(300.0f)
					[
						SAssignNew(LockSearchBox, SSearchBox)
						.HintText(LOCTEXT("SearchLocks", "Search Locks"))
						.ToolTipText(LOCTEXT("PlasticLocksSearch_Tooltip", "Filter the list of locks by keyword."))
						.OnTextChanged(this, &SPlasticSourceControlLocksWidget::OnSearchTextChanged)
					]
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					// Button to Configure Lock Rules in the cloud (only enabled for a cloud repository)
					SNew(SButton)
					.ContentPadding(FMargin(6.0f, 0.0f))
					.IsEnabled(!OrganizationName.IsEmpty())
					.ToolTipText(OrganizationName.IsEmpty() ?
						LOCTEXT("PlasticLockRulesURLTooltipDisabled", "Web link to the Unity Dashboard disabled. Only available for Cloud repositories.") :
						LOCTEXT("PlasticLockRulesURLTooltipEnabled", "Navigate to lock rules configuration page in the Unity Dashboard."))
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.OnClicked(this, &SPlasticSourceControlLocksWidget::OnConfigureLockRulesClicked, OrganizationName)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("PropertyWindow.Locked"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
							.Text(LOCTEXT("ConfigureLockRules", "Configure rules"))
						]
					]
				]
			]
		]
		+SVerticalBox::Slot() // The main content: the list of locks
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				CreateContentPanel()
			]
			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.FillHeight(1.0f)
			[
				// Text to display when there is no lock displayed
				SNew(STextBlock)
				.Text(LOCTEXT("NoLock", "There is no lock to display."))
				.Visibility_Lambda([this]() { return SourceControlLocks.Num() ? EVisibility::Collapsed : EVisibility::Visible; })
			]
		]
		+SVerticalBox::Slot() // Status bar (Always visible)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(FMargin(0.f, 3.f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return RefreshStatus; })
					.Margin(FMargin(5.f, 0.f))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return FText::FromString(WorkspaceSelector); })
					.ToolTipText(LOCTEXT("PlasticBranchCurrent_Tooltip", "Current branch."))
				]
			]
		]
	];
}

TSharedRef<SWidget> SPlasticSourceControlLocksWidget::CreateToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			bShouldRefresh = true;
			bShouldInvalidateLocksCache = true;
		})),
		NAME_None,
		LOCTEXT("SourceControl_RefreshButton", "Refresh"),
		LOCTEXT("SourceControl_RefreshButton_Tooltip", "Refreshes locks from revision control provider."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Refresh"));

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SPlasticSourceControlLocksWidget::CreateContentPanel()
{
	// Inspired by Engine\Source\Editor\SourceControlWindows\Private\SSourceControlChangelists.cpp
	// TSharedRef<SListView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateChangelistFilesView()

	UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
	if (!Settings->bShowLockIdColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlLocksListViewColumn::ItemId::Id());
	}
	if (!Settings->bShowLockWorkspaceColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlLocksListViewColumn::Workspace::Id());
	}
	if (!Settings->bShowLockDateColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlLocksListViewColumn::Date::Id());
	}
	if (!Settings->bShowLockDestinationBranchColumn)
	{
		HiddenColumnsList.Add(PlasticSourceControlLocksListViewColumn::DestinationBranch::Id());
	}

	TSharedRef<SListView<FPlasticSourceControlLockRef>> LockView = SNew(SListView<FPlasticSourceControlLockRef>)
		.ListItemsSource(&LockRows)
		.OnGenerateRow(this, &SPlasticSourceControlLocksWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SPlasticSourceControlLocksWidget::OnOpenContextMenu)
		.OnItemToString_Debug_Lambda([this](FPlasticSourceControlLockRef Lock) { return Lock->Path; })
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList(HiddenColumnsList)
			.OnHiddenColumnsListChanged(this, &SPlasticSourceControlLocksWidget::OnHiddenColumnsListChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::ItemId::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::ItemId::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::ItemId::GetToolTipText())
			.FillWidth(0.5f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::ItemId::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::ItemId::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Path::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Path::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Path::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(4.0f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Path::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Path::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Status::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Status::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Status::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(0.5f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Status::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Status::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Date::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Date::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Date::GetToolTipText())
			.FillWidth(1.5f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Date::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Date::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Owner::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Owner::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Owner::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Owner::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Owner::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::DestinationBranch::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::DestinationBranch::GetToolTipText())
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Branch::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Branch::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Branch::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(2.0f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Branch::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Branch::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(PlasticSourceControlLocksListViewColumn::Workspace::Id())
			.DefaultLabel(PlasticSourceControlLocksListViewColumn::Workspace::GetDisplayText())
			.DefaultTooltip(PlasticSourceControlLocksListViewColumn::Workspace::GetToolTipText())
			.FillWidth(1.5f)
			.SortPriority(this, &SPlasticSourceControlLocksWidget::GetColumnSortPriority, PlasticSourceControlLocksListViewColumn::Workspace::Id())
			.SortMode(this, &SPlasticSourceControlLocksWidget::GetColumnSortMode, PlasticSourceControlLocksListViewColumn::Workspace::Id())
			.OnSort(this, &SPlasticSourceControlLocksWidget::OnColumnSortModeChanged)
		);

	LocksListView = LockView;

	return LockView;
}

TSharedRef<ITableRow> SPlasticSourceControlLocksWidget::OnGenerateRow(FPlasticSourceControlLockRef InLock, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SPlasticSourceControlLockRow, OwnerTable)
		.LockToVisualize(InLock)
		.HighlightText_Lambda([this]() { return LockSearchBox->GetText(); });
}

void SPlasticSourceControlLocksWidget::OnHiddenColumnsListChanged()
{
	// Update and save config to reload it on the next Editor sessions
	if (LocksListView && LocksListView->GetHeaderRow())
	{
		UPlasticSourceControlProjectSettings* Settings = GetMutableDefault<UPlasticSourceControlProjectSettings>();
		Settings->bShowLockIdColumn = true;
		Settings->bShowLockWorkspaceColumn = true;
		Settings->bShowLockDateColumn = true;
		Settings->bShowLockDestinationBranchColumn = true;

		for (const FName& ColumnId : LocksListView->GetHeaderRow()->GetHiddenColumnIds())
		{
			if (ColumnId == PlasticSourceControlLocksListViewColumn::ItemId::Id())
			{
				Settings->bShowLockIdColumn = false;
			}
			else if (ColumnId == PlasticSourceControlLocksListViewColumn::Workspace::Id())
			{
				Settings->bShowLockWorkspaceColumn = false;
			}
			else if (ColumnId == PlasticSourceControlLocksListViewColumn::Date::Id())
			{
				Settings->bShowLockDateColumn = false;
			}
			else if (ColumnId == PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
			{
				Settings->bShowLockDestinationBranchColumn = false;
			}
		}
		Settings->SaveConfig();
	}
}

void SPlasticSourceControlLocksWidget::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	LockSearchBox->SetError(SearchTextFilter->GetFilterErrorText());
}

void SPlasticSourceControlLocksWidget::PopulateItemSearchStrings(const FPlasticSourceControlLock& InItem, TArray<FString>& OutStrings)
{
	InItem.PopulateSearchString(OutStrings);
}

void SPlasticSourceControlLocksWidget::OnRefreshUI()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlLocksWidget::OnRefreshUI);

	const int32 ItemCount = SourceControlLocks.Num();
	LockRows.Empty(ItemCount);
	for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
	{
		const FPlasticSourceControlLockRef& Item = SourceControlLocks[ItemIndex];
		if (SearchTextFilter->PassesFilter(Item.Get()))
		{
			LockRows.Emplace(Item);
		}
	}

	if (GetListView())
	{
		SortLockView();
		GetListView()->RequestListRefresh();
	}
}

EColumnSortPriority::Type SPlasticSourceControlLocksWidget::GetColumnSortPriority(const FName InColumnId) const
{
	if (InColumnId == PrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (InColumnId == SecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

EColumnSortMode::Type SPlasticSourceControlLocksWidget::GetColumnSortMode(const FName InColumnId) const
{
	if (InColumnId == PrimarySortedColumn)
	{
		return PrimarySortMode;
	}
	else if (InColumnId == SecondarySortedColumn)
	{
		return SecondarySortMode;
	}

	return EColumnSortMode::None;
}

void SPlasticSourceControlLocksWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	if (InSortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedColumn = InColumnId;
		PrimarySortMode = InSortMode;

		if (InColumnId == SecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			SecondarySortedColumn = FName();
			SecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (InSortPriority == EColumnSortPriority::Secondary)
	{
		SecondarySortedColumn = InColumnId;
		SecondarySortMode = InSortMode;
	}

	if (GetListView())
	{
		SortLockView();
		GetListView()->RequestListRefresh();
	}
}

void SPlasticSourceControlLocksWidget::SortLockView()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlLocksWidget::SortLockView);

	if (PrimarySortedColumn.IsNone() || LockRows.Num() == 0)
	{
		return; // No column selected for sorting or nothing to sort.
	}

	auto CompareItemIds = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return Lhs->ItemId < Rhs->ItemId ? -1 : (Lhs->ItemId == Rhs->ItemId ? 0 : 1);
	};

	auto CompareStatuses = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return FCString::Stricmp(*Lhs->Status, *Rhs->Status);
	};

	auto ComparePaths = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return UE::ComparisonUtility::CompareNaturalOrder(*Lhs->Path, *Rhs->Path);
	};

	auto CompareOwners = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return FCString::Stricmp(*Lhs->Owner, *Rhs->Owner);
	};

	auto CompareDestinationBranches = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return FCString::Stricmp(*Lhs->DestinationBranch, *Rhs->DestinationBranch);
	};

	auto CompareBranches = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return FCString::Stricmp(*Lhs->Branch, *Rhs->Branch);
	};

	auto CompareWorkspaces = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return FCString::Stricmp(*Lhs->Workspace, *Rhs->Workspace);
	};

	auto CompareDates = [](const FPlasticSourceControlLock* Lhs, const FPlasticSourceControlLock* Rhs)
	{
		return Lhs->Date < Rhs->Date ? -1 : (Lhs->Date == Rhs->Date ? 0 : 1);
	};

	auto GetCompareFunc = [&](const FName& ColumnId)
	{
		if (ColumnId == PlasticSourceControlLocksListViewColumn::ItemId::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareItemIds);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Status::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareStatuses);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Path::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(ComparePaths);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Owner::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareOwners);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::DestinationBranch::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareDestinationBranches);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Branch::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareBranches);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Workspace::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareWorkspaces);
		}
		else if (ColumnId == PlasticSourceControlLocksListViewColumn::Date::Id())
		{
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>(CompareDates);
		}
		else
		{
			checkNoEntry();
			return TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)>();
		};
	};

	TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)> PrimaryCompare = GetCompareFunc(PrimarySortedColumn);
	TFunction<int32(const FPlasticSourceControlLock*, const FPlasticSourceControlLock*)> SecondaryCompare;
	if (!SecondarySortedColumn.IsNone())
	{
		SecondaryCompare = GetCompareFunc(SecondarySortedColumn);
	}

	if (PrimarySortMode == EColumnSortMode::Ascending)
	{
		// NOTE: StableSort() would give a better experience when the sorted columns(s) has the same values and new values gets added, but it is slower
		//       with large changelists (7600 items was about 1.8x slower in average measured with Unreal Insight). Because this code runs in the main
		//       thread and can be invoked a lot, the trade off went if favor of speed.
		LockRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlLockPtr& Lhs, const FPlasticSourceControlLockPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get()));
			if (Result < 0)
			{
				return true;
			}
			else if (Result > 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get())) > 0;
			}
		});
	}
	else
	{
		LockRows.Sort([this, &PrimaryCompare, &SecondaryCompare](const FPlasticSourceControlLockPtr& Lhs, const FPlasticSourceControlLockPtr& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get()));
			if (Result > 0)
			{
				return true;
			}
			else if (Result < 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get())) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<FPlasticSourceControlLock*>(Lhs.Get()), static_cast<FPlasticSourceControlLock*>(Rhs.Get())) > 0;
			}
		});
	}
}

TSharedPtr<SWidget> SPlasticSourceControlLocksWidget::OnOpenContextMenu()
{
	const TArray<FPlasticSourceControlLockRef> SelectedLocks = LocksListView->GetSelectedItems();
	if (SelectedLocks.Num() == 0)
	{
		return nullptr;
	}

	// Check to see if any of these locks are releasable, that is, if some of them are "Locked" instead of simply being "Retained"
	bool bCanReleaseLocks = false;
	for (const FPlasticSourceControlLockRef& SelectedLock : SelectedLocks)
	{
		if (SelectedLock->bIsLocked)
		{
			bCanReleaseLocks = true;
			break;
		}
	}

	static const FText SelectASingleLockTooltip(LOCTEXT("SelectASingleLockTooltip", "Select a single lock."));
	static const FText UpdateUVCSTooltip(LOCTEXT("MergeLockXmlTooltip", "Update Unity Version Control (PlasticSCM) to 11.0.16.8101 with SmartLocks or later."));

	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName = "PlasticSourceControl.LocksContextMenu";
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	// Build up the menu
	FToolMenuContext Context;
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");

	Section.AddMenuEntry(
		"ReleaseLock",
		LOCTEXT("ReleaseLock", "Release"),
		LOCTEXT("ReleaseLocksTooltip", "Release Lock(s) on the selected assets.\nReleasing locks will allow other users to keep working on these files and retrieve locks (on the same branch, in the latest revision)."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlLocksWidget::OnReleaseLocksClicked, SelectedLocks),
			FCanExecuteAction::CreateLambda([bCanReleaseLocks]() { return bCanReleaseLocks; })
		)
	);
	Section.AddMenuEntry(
		"RemoveLock",
		LOCTEXT("RemoveLock", "Remove"),
		LOCTEXT("RemoveLocksTooltip", "Remove Lock(s) on the selected assets.\nRemoving locks will allow other users to edit these files anywhere (on any branch) increasing the risk of future merge conflicts."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPlasticSourceControlLocksWidget::OnRemoveLocksClicked, SelectedLocks),
			FCanExecuteAction()
		)
	);

	return ToolMenus->GenerateWidget(Menu);
}

FReply SPlasticSourceControlLocksWidget::OnConfigureLockRulesClicked(const FString InOrganizationName)
{
	PlasticSourceControlUtils::OpenLockRulesInCloudDashboard(InOrganizationName);
	return FReply::Handled();
}

void SPlasticSourceControlLocksWidget::OnReleaseLocksClicked(TArray<FPlasticSourceControlLockRef> InSelectedLocks)
{
	ExecuteUnlock(MoveTemp(InSelectedLocks), false);
}

void SPlasticSourceControlLocksWidget::OnRemoveLocksClicked(TArray<FPlasticSourceControlLockRef> InSelectedLocks)
{
	ExecuteUnlock(MoveTemp(InSelectedLocks), true);
}

void SPlasticSourceControlLocksWidget::ExecuteUnlock(TArray<FPlasticSourceControlLockRef>&& InSelectedLocks, const bool bInRemove)
{
	const FText UnlockQuestion = FText::Format(bInRemove ?
		LOCTEXT("RemoveLocksDialog", "Removing locks will allow other users to edit these files anywhere (on any branch) increasing the risk of future merge conflicts. Would you like to remove {0} lock(s)?") :
		LOCTEXT("ReleaseLocksDialog", "Releasing locks will allow other users to keep working on these files and retrieve locks (on the same branch, in the latest revision). Would you like to release {0} lock(s)?"),
		FText::AsNumber(InSelectedLocks.Num()));
	const EAppReturnType::Type Choice = FMessageDialog::Open(
		EAppMsgCategory::Info,
		EAppMsgType::YesNo, UnlockQuestion
		, bInRemove ? LOCTEXT("RemoveLocksTitle", "Remove Lock(s)?") : LOCTEXT("ReleaseLocksTitle", "Release Lock(s)?")
	);
	if (Choice == EAppReturnType::Yes)
	{
		if (!Notification.IsInProgress())
		{
			// Launch a custom "Unlock" operation
			FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
			const FString& WorkspaceRoot = Provider.GetPathToWorkspaceRoot();
			const TArray<FString> Files = PlasticSourceControlUtils::LocksToFileNames(WorkspaceRoot, InSelectedLocks);
			TSharedRef<FPlasticUnlock, ESPMode::ThreadSafe> UnlockOperation = ISourceControlOperation::Create<FPlasticUnlock>();
			UnlockOperation->bRemove = bInRemove;
			UnlockOperation->Locks = MoveTemp(InSelectedLocks);
			const ECommandResult::Type Result = Provider.Execute(UnlockOperation, Files, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlLocksWidget::OnUnlockOperationComplete));
			if (Result == ECommandResult::Succeeded)
			{
				// Display an ongoing notification during the whole operation (packages will be reloaded at the completion of the operation)
				Notification.DisplayInProgress(UnlockOperation->GetInProgressString());
				StartRefreshStatus();
			}
			else
			{
				// Report failure with a notification (but nothing need to be reloaded since no local change is expected)
				FNotification::DisplayFailure(UnlockOperation.Get());
			}
		}
		else
		{
			FMessageLog SourceControlLog("SourceControl");
			SourceControlLog.Warning(LOCTEXT("SourceControlMenu_InProgress", "Source control operation already in progress"));
			SourceControlLog.Notify();
		}
	}
}

void SPlasticSourceControlLocksWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	// Detect transitions of the source control being available/unavailable. Ex: When the user changes the source control in UI, the provider gets selected,
	// but it is not connected/available until the user accepts the settings. The source control doesn't have callback for availability and we want to refresh everything
	// once it gets available.
	if (ISourceControlModule::Get().IsEnabled() && !bSourceControlAvailable && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		bSourceControlAvailable = true;
		bShouldRefresh = true;
	}

	if (bShouldRefresh)
	{
		RequestLocksRefresh(bShouldInvalidateLocksCache);
		bShouldRefresh = false;
		bShouldInvalidateLocksCache = false;
	}

	if (bIsRefreshing)
	{
		TickRefreshStatus(InDeltaTime);
	}
}

void SPlasticSourceControlLocksWidget::StartRefreshStatus()
{
	if (!bIsRefreshing)
	{
		bIsRefreshing = true;
		RefreshStatusStartSecs = FPlatformTime::Seconds();
	}
}

void SPlasticSourceControlLocksWidget::TickRefreshStatus(double InDeltaTime)
{
	const int32 RefreshStatusTimeElapsed = static_cast<int32>(FPlatformTime::Seconds() - RefreshStatusStartSecs);
	RefreshStatus = FText::Format(LOCTEXT("PlasticSourceControl_RefreshLocks", "Refreshing locks... ({0} s)"), FText::AsNumber(RefreshStatusTimeElapsed));
}

void SPlasticSourceControlLocksWidget::EndRefreshStatus()
{
	bIsRefreshing = false;
	RefreshStatus = FText::GetEmpty();
}

void SPlasticSourceControlLocksWidget::RequestLocksRefresh(const bool bInInvalidateLocksCache)
{
	if (!ISourceControlModule::Get().IsEnabled() || (!FPlasticSourceControlModule::Get().GetProvider().IsAvailable()))
	{
		return;
	}

	StartRefreshStatus();

	if (bInInvalidateLocksCache)
	{
		PlasticSourceControlUtils::InvalidateLocksCache();
	}

	TSharedRef<FPlasticGetLocks, ESPMode::ThreadSafe> GetLocksOperation = ISourceControlOperation::Create<FPlasticGetLocks>();

	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	Provider.Execute(GetLocksOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SPlasticSourceControlLocksWidget::OnGetLocksOperationComplete));
}

void SPlasticSourceControlLocksWidget::OnGetLocksOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPlasticSourceControlLocksWidget::OnGetLocksOperationComplete);

	TSharedRef<FPlasticGetLocks, ESPMode::ThreadSafe> OperationGetLocks = StaticCastSharedRef<FPlasticGetLocks>(InOperation);
	SourceControlLocks = MoveTemp(OperationGetLocks->Locks);

	WorkspaceSelector = FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector();

	EndRefreshStatus();
	OnRefreshUI();
}

void SPlasticSourceControlLocksWidget::OnUnlockOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	// Ask for a full refresh of the list of locks (and don't call EndRefreshStatus() yet)
	bShouldRefresh = true;

	Notification.RemoveInProgress();

	FNotification::DisplayResult(InOperation, InResult);
}

void SPlasticSourceControlLocksWidget::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	bSourceControlAvailable = NewProvider.IsAvailable(); // Check if it is connected.
	bShouldRefresh = true;

	if (&NewProvider != &OldProvider)
	{
		LockRows.Reset();
		if (GetListView())
		{
			GetListView()->RequestListRefresh();
		}
	}
}

void SPlasticSourceControlLocksWidget::HandleSourceControlStateChanged()
{
	bShouldRefresh = true;
	if (GetListView())
	{
		GetListView()->RequestListRefresh();
	}
}

FReply SPlasticSourceControlLocksWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F5)
	{
		// Pressing F5 refreshes the list of locks
		bShouldRefresh = true;
		bShouldInvalidateLocksCache = true;
		return FReply::Handled();
	}
	else if ((InKeyEvent.GetKey() == EKeys::Delete) || (InKeyEvent.GetKey() == EKeys::BackSpace))
	{
		// Pressing Delete or BackSpace removes the selected locks
		const TArray<FPlasticSourceControlLockRef> SelectedLocks = LocksListView->GetSelectedItems();
		if (SelectedLocks.Num() > 0)
		{
			OnRemoveLocksClicked(SelectedLocks);
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
