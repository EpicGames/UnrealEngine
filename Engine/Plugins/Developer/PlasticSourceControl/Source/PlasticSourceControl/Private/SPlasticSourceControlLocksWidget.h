// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Notification.h"

#include "Misc/TextFilter.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"

typedef TSharedRef<class FPlasticSourceControlLock, ESPMode::ThreadSafe> FPlasticSourceControlLockRef;

class SSearchBox;

// Widget displaying the list of locks in the tab window, see FPlasticSourceControlLocksWindow
class SPlasticSourceControlLocksWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPlasticSourceControlLocksWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	TSharedRef<SWidget> CreateToolBar();
	TSharedRef<SWidget> CreateContentPanel();

	TSharedRef<ITableRow> OnGenerateRow(FPlasticSourceControlLockRef InLock, const TSharedRef<STableViewBase>& OwnerTable);
	void OnHiddenColumnsListChanged();

	void OnSearchTextChanged(const FText& InFilterText);
	void PopulateItemSearchStrings(const FPlasticSourceControlLock& InItem, TArray<FString>& OutStrings);

	void OnRefreshUI();

	EColumnSortPriority::Type GetColumnSortPriority(const FName InColumnId) const;
	EColumnSortMode::Type GetColumnSortMode(const FName InColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode);

	void SortLockView();

	TSharedPtr<SWidget> OnOpenContextMenu();

	FReply OnConfigureLockRulesClicked(const FString InOrganizationName);

	void OnReleaseLocksClicked(TArray<FPlasticSourceControlLockRef> InSelectedLocks);
	void OnRemoveLocksClicked(TArray<FPlasticSourceControlLockRef> InSelectedLocks);
	void ExecuteUnlock(TArray<FPlasticSourceControlLockRef>&& InSelectedLocks, const bool bInRemove);

	void StartRefreshStatus();
	void TickRefreshStatus(double InDeltaTime);
	void EndRefreshStatus();

	void RequestLocksRefresh(const bool bInInvalidateLocksCache);

	/** Source control callbacks */
	void OnGetLocksOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnUnlockOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

	/** Delegate handler for when source control state changes */
	void HandleSourceControlStateChanged();

	SListView<FPlasticSourceControlLockRef>* GetListView() const
	{
		return LocksListView.Get();
	}

	/** Interpret F5, Enter and Delete keys */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TSharedPtr<SSearchBox> LockSearchBox;

	FName PrimarySortedColumn;
	FName SecondarySortedColumn;
	EColumnSortMode::Type PrimarySortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type SecondarySortMode = EColumnSortMode::None;

	TArray<FName> HiddenColumnsList;

	bool bShouldRefresh = false;
	bool bShouldInvalidateLocksCache = false;
	bool bSourceControlAvailable = false;

	FText RefreshStatus;
	bool bIsRefreshing = false;
	double RefreshStatusStartSecs;

	FString WorkspaceSelector;

	/** Ongoing notification for a long-running asynchronous source control operation, if any */
	FNotification Notification;

	TSharedPtr<SListView<FPlasticSourceControlLockRef>> LocksListView;
	TSharedPtr<TTextFilter<const FPlasticSourceControlLock&>> SearchTextFilter;

	TArray<FPlasticSourceControlLockRef> SourceControlLocks; // Full list from source (filtered by date)
	TArray<FPlasticSourceControlLockRef> LockRows; // Filtered list to display based on the search text filter

	/** Delegate handle for the HandleSourceControlStateChanged function callback */
	FDelegateHandle SourceControlStateChangedDelegateHandle;
};
