// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlRenameBranch.h"

#include "SPlasticSourceControlBranchesWidget.h"

#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControlBranchesWindow"

// Extract the short name of the branch, after the last slash
static FString GetShortBranchName(const FString& InBranchName)
{
	int32 LastSlashIndex;
	if (InBranchName.FindLastChar(TEXT('/'), LastSlashIndex))
	{
		return InBranchName.RightChop(LastSlashIndex + 1);
	}
	return FString();
}

// Extract the name of the parent branch, if any, before and including the last slash
static FString GetParentBranchName(const FString& InBranchName)
{
	int32 LastSlashIndex;
	if (InBranchName.FindLastChar(TEXT('/'), LastSlashIndex))
	{
		return InBranchName.Left(LastSlashIndex + 1);
	}
	return FString();
}

void SPlasticSourceControlRenameBranch::Construct(const FArguments& InArgs)
{
	BranchesWidget = InArgs._BranchesWidget;
	ParentWindow = InArgs._ParentWindow;
	OldBranchName = InArgs._OldBranchName;

	NewBranchName = GetShortBranchName(OldBranchName);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("PlasticRenameBrancheDetails", "Rename branch {0}"), FText::FromString(OldBranchName)))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f, 5.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("PlasticRenameBrancheNameTooltip", "Enter a new name for the branch"))
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlasticRenameBrancheNameLabel", "New name: "))
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SAssignNew(BranchNameTextBox, SEditableTextBox)
				.Text(FText::FromString(NewBranchName))
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					NewBranchName = InText.ToString();
				})
				.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
				{
					NewBranchName = InText.ToString();
					if (TextCommitType == ETextCommit::OnEnter)
					{
						RenamedClicked();
					}
				})
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Rename", "Rename"))
				.IsEnabled(this, &SPlasticSourceControlRenameBranch::CanRenameBranch)
				.ToolTipText(this, &SPlasticSourceControlRenameBranch::RenameButtonTooltip)
				.OnClicked(this, &SPlasticSourceControlRenameBranch::RenamedClicked)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SPlasticSourceControlRenameBranch::CancelClicked)
			]
		]
	];

	ParentWindow.Pin()->SetWidgetToFocusOnActivate(BranchNameTextBox);
}

bool SPlasticSourceControlRenameBranch::CanRenameBranch() const
{
	if (NewBranchName.IsEmpty())
	{
		return false;
	}

	if (NewBranchName == GetShortBranchName(OldBranchName))
	{
		return false;
	}

	return SPlasticSourceControlBranchesWidget::IsBranchNameValid(NewBranchName);
}

FText SPlasticSourceControlRenameBranch::RenameButtonTooltip() const
{
	if (NewBranchName.IsEmpty())
	{
		return LOCTEXT("RenameEmpty_Tooltip", "Enter a name for the branch.");
	}

	if (NewBranchName == GetShortBranchName(OldBranchName))
	{
		return LOCTEXT("RenameSame_Tooltip", "Enter a new name for the branch.");
	}

	if (!SPlasticSourceControlBranchesWidget::IsBranchNameValid(NewBranchName))
	{
		return LOCTEXT("RenameInvalid_Tooltip", "Branch name cannot contain any of the following characters: @#/:\"?'\\n\\r\\t");
	}

	return FText::Format(LOCTEXT("RenameBranch_Tooltip", "Rename branch {0}\nto {1}."),
		FText::FromString(OldBranchName),
		FText::FromString(GetParentBranchName(OldBranchName) + NewBranchName));
}

FReply SPlasticSourceControlRenameBranch::RenamedClicked()
{
	if (TSharedPtr<SPlasticSourceControlBranchesWidget> Branches = BranchesWidget.Pin())
	{
		Branches->RenameBranch(OldBranchName, NewBranchName);
	}

	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlRenameBranch::CancelClicked()
{
	if (TSharedPtr<SWindow> ParentWindowPin = ParentWindow.Pin())
	{
		ParentWindowPin->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SPlasticSourceControlRenameBranch::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Pressing Escape returns as if the user clicked Cancel
		return CancelClicked();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
