// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
class FReply;
class SPlasticSourceControlBranchesWidget;
class SWindow;

class SPlasticSourceControlRenameBranch : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlRenameBranch)
		: _BranchesWidget()
		, _ParentWindow()
		, _OldBranchName()
	{}

		SLATE_ARGUMENT(TSharedPtr<SPlasticSourceControlBranchesWidget>, BranchesWidget)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(FString, OldBranchName)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	bool CanRenameBranch() const;
	FText RenameButtonTooltip() const;

	FReply RenamedClicked();
	FReply CancelClicked();

	/** Interpret Escape as Cancel */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	FString OldBranchName;
	FString NewBranchName;

	TSharedPtr<SEditableTextBox> BranchNameTextBox;

	TWeakPtr<SPlasticSourceControlBranchesWidget> BranchesWidget;
	TWeakPtr<SWindow> ParentWindow;
};
