// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlSettings.h"
#include "PlasticSourceControlOperations.h"
#include "PlasticSourceControlModule.h"
#include "PlasticSourceControlUtils.h"

#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SPlasticSourceControlSettings"

void SPlasticSourceControlSettings::Construct(const FArguments& InArgs)
{
	const FSlateFontInfo Font = FAppStyle::GetFontStyle(TEXT("SourceControl.LoginWindow.Font"));

	bAutoCreateIgnoreFile = CanAutoCreateIgnoreFile();
	WorkspaceParams.bAutoInitialCommit = true;

	WorkspaceParams.InitialCommitMessage = LOCTEXT("InitialCommitMessage", "Initial checkin");
	WorkspaceParams.ServerUrl = FText::FromString(FPlasticSourceControlModule::Get().GetProvider().GetServerUrl());

	if (FApp::HasProjectName())
	{
		WorkspaceParams.WorkspaceName = FText::FromString(FApp::GetProjectName());
		WorkspaceParams.RepositoryName = WorkspaceParams.WorkspaceName;
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		// Versions (Plugin & Unity Version Control) useful eg to help diagnose issues from screenshots
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("PlasticVersions_Tooltip", "Unity Version Control (formerly Plastic SCM) and Plugin versions"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlasticVersions", "Unity Version Control"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SPlasticSourceControlSettings::GetVersions)
				.Font(Font)
			]
		]
		// Unity Version Control command line tool not available warning and download link
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Visibility(this, &SPlasticSourceControlSettings::PlasticNotAvailable)
			.ToolTipText(LOCTEXT("PlasticNotAvailable_Tooltip", "Failed to launch Unity Version Control 'cm' command line tool.\nYou need to install it and make sure it is correctly configured with your credentials."))
			.Text(LOCTEXT("PlasticNotAvailable", "Unity Version Control Command Line tool 'cm' failed to start."))
			.Font(Font)
			.ColorAndOpacity(FLinearColor::Red)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SHyperlink)
				.Visibility(this, &SPlasticSourceControlSettings::PlasticNotAvailable)
				.ToolTipText(LOCTEXT("PlasticDownload_Tooltip", "Download Unity Version Control (Plastic SCM)"))
				.Text(LOCTEXT("PlasticDownload", "https://www.plasticscm.com/download/downloadinstaller/..."))
				.OnNavigate(FSimpleDelegate::CreateLambda([]()
				{
#if PLATFORM_WINDOWS
					FPlatformProcess::LaunchURL(TEXT("https://www.plasticscm.com/download/downloadinstaller/last/plasticscm/windows/cloudedition"), NULL, NULL);
#elif PLATFORM_MAC
					FPlatformProcess::LaunchURL(TEXT("https://www.plasticscm.com/download/downloadinstaller/last/plasticscm/macosx/cloudedition"), NULL, NULL);
#elif PLATFORM_LINUX
					FPlatformProcess::LaunchURL(TEXT("https://www.plasticscm.com/plastic-for-linux"), NULL, NULL);
#endif
				}))
			]
		]
		// Path to the Unity Version Control binary
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("BinaryPathLabel_Tooltip", "Path to the Unity Version Control Command Line tool 'cm' executable"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PathLabel", "Path to cm"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPlasticSourceControlSettings::GetBinaryPathText)
				.HintText(LOCTEXT("BinaryPathLabel", "Path to the Unity Version Control 'cm' executable"))
				.OnTextCommitted(this, &SPlasticSourceControlSettings::OnBinaryPathTextCommited)
				.Font(Font)
			]
		]
		// Root of the workspace
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(this, &SPlasticSourceControlSettings::GetPathToWorkspaceRoot)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WorkspaceRootLabel", "Root of the workspace"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SPlasticSourceControlSettings::GetPathToWorkspaceRoot)
				.Font(Font)
			]
		]
		// User Name
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("PlasticUserName_Tooltip", "User name configured for the Unity Version Control workspace"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlasticUserName", "User Name"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SPlasticSourceControlSettings::GetUserName)
				.Font(Font)
			]
		]
		// Separator
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
		]
		// Explanation text
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f, 5.0f)
		[
			SNew(STextBlock)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			.ToolTipText(LOCTEXT("WorkspaceNotFound_Tooltip", "No Workspace found at the level or above the current Project. Use the form to create a new one."))
			.Text(LOCTEXT("WorkspaceNotFound", "Current Project is not in a Unity Version Control Workspace. Create a new one:"))
			.Font(Font)
		]
		// Workspace and Repository Name
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WorkspaceRepositoryName", "Workspace & Repository"))
				.ToolTipText(LOCTEXT("WorkspaceRepositoryName_Tooltip", "Enter the Name of the new Workspace and Repository to create or use"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPlasticSourceControlSettings::GetWorkspaceName)
				.ToolTipText(LOCTEXT("WorkspaceName_Tooltip", "Enter the Name of the new Workspace to create"))
				.HintText(LOCTEXT("WorkspaceName_Hint", "Name of the Workspace to create"))
				.OnTextCommitted(this, &SPlasticSourceControlSettings::OnWorkspaceNameCommited)
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPlasticSourceControlSettings::GetRepositoryName)
				.ToolTipText(LOCTEXT("RepositoryName_Tooltip", "Enter the Name of the Repository to use or create"))
				.HintText(LOCTEXT("RepositoryName_Hint", "Name of the Repository to use or create"))
				.OnTextCommitted(this, &SPlasticSourceControlSettings::OnRepositoryNameCommited)
				.Font(Font)
			]
		]
		// Server URL address:port
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			.ToolTipText(LOCTEXT("ServerUrl_Tooltip", "Enter the Server URL in the form address:port (eg. YourOrganization@cloud, local, or something like ip:port, eg localhost:8087)"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ServerUrl", "Server URL address:port"))
				.Font(Font)
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(SEditableTextBox)
				.Text(this, &SPlasticSourceControlSettings::GetServerUrl)
				.HintText(LOCTEXT("EnterServerUrl", "Enter the Server URL"))
				.OnTextCommitted(this, &SPlasticSourceControlSettings::OnServerUrlCommited)
				.Font(Font)
			]
		]
		// Option to create a Partial/Gluon Workspace designed
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			.ToolTipText(LOCTEXT("CreatePartialWorkspace_Tooltip", "Create the new workspace in Gluon/partial mode, designed for artists."))
			.IsChecked(WorkspaceParams.bCreatePartialWorkspace)
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedCreatePartialWorkspace)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CreatePartialWorkspace", "Make the new workspace a Gluon partial workspace."))
				.Font(Font)
			]
		]
		// Option to add a 'ignore.conf' file at Workspace creation time
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			.ToolTipText(LOCTEXT("CreateIgnoreFile_Tooltip", "Create and add a standard 'ignore.conf' file"))
			.IsEnabled(this, &SPlasticSourceControlSettings::CanAutoCreateIgnoreFile)
			.IsChecked(bAutoCreateIgnoreFile)
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedCreateIgnoreFile)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CreateIgnoreFile", "Add a ignore.conf file"))
				.Font(Font)
			]
		]
		// Option to Make the initial Unity Version Control checkin
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			.ToolTipText(LOCTEXT("InitialCommit_Tooltip", "Make the initial Unity Version Control checkin"))
			+SHorizontalBox::Slot()
			.FillWidth(0.7f)
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedInitialCommit)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InitialCommit", "Initial Checkin"))
					.Font(Font)
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.4f)
			.Padding(2.0f)
			[
				SNew(SMultiLineEditableTextBox)
				.Text(this, &SPlasticSourceControlSettings::GetInitialCommitMessage)
				.HintText(LOCTEXT("InitialCommitMessage_Hint", "Message for the initial checkin"))
				.OnTextCommitted(this, &SPlasticSourceControlSettings::OnInitialCommitMessageCommited)
				.Font(Font)
			]
		]
		// Option to run an Update Status operation at Editor Startup
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(SPlasticSourceControlSettings::IsUpdateStatusAtStartupChecked())
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedUpdateStatusAtStartup)
			.ToolTipText(LOCTEXT("UpdateStatusAtStartup_Tooltip", "Run an asynchronous Update Status at Editor startup (can be slow)."))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UpdateStatusAtStartup", "Update workspace Status at Editor startup"))
				.Font(Font)
			]
		]
		// Option to call History as part of Update Status operation to check for potential recent changes in other branches
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("UpdateStatusOtherBranches_Tooltip", "Enable Update status to detect more recent changes on other branches in order to display warnings (can be slow)."))
			.IsChecked(SPlasticSourceControlSettings::IsUpdateStatusOtherBranchesChecked())
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedUpdateStatusOtherBranches)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UpdateStatusOtherBranches", "Update Status also checks history to detect changes on other branches."))
				.Font(Font)
			]
		]
		// Option for the View Changes (Changelists) window to also show locally Changed and Private files
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("ViewLocalChanges_Tooltip", "Enable the \"View Changes\" window to search for and show locally Changed and Private files (can be slow)."))
			.IsChecked(SPlasticSourceControlSettings::IsViewLocalChangesChecked())
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedViewLocalChanges)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ViewLocalChanges", "Show local Changes in the \"View Changes\" window."))
				.Font(Font)
			]
		]
		// Option to enable Source Control Verbose logs
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("EnableVerboseLogs_Tooltip", "Override LogSourceControl default verbosity level to Verbose (except if already set to VeryVerbose)."))
			.IsChecked(SPlasticSourceControlSettings::IsEnableVerboseLogsChecked())
			.OnCheckStateChanged(this, &SPlasticSourceControlSettings::OnCheckedEnableVerboseLogs)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EnableVerboseLogs", "Enable Revision Control Verbose logs"))
				.Font(Font)
			]
		]
		// Button to create a new Workspace
		+SVerticalBox::Slot()
		.FillHeight(2.5f)
		.Padding(4.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanCreatePlasticWorkspace)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SButton)
				.IsEnabled(this, &SPlasticSourceControlSettings::IsReadyToCreatePlasticWorkspace)
				.Text(LOCTEXT("PlasticInitWorkspace", "Create a new Unity Version Control workspace for the current project"))
				.ToolTipText(LOCTEXT("PlasticInitWorkspace_Tooltip", "Create a new Unity Version Control repository and workspace and for the current project"))
				.OnClicked(this, &SPlasticSourceControlSettings::OnClickedCreatePlasticWorkspace)
				.HAlign(HAlign_Center)
				.ContentPadding(6.0f)
			]
		]
		// Button to add a 'ignore.conf' file on an existing Workspace
		+SVerticalBox::Slot()
		.FillHeight(2.0f)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SPlasticSourceControlSettings::CanAddIgnoreFile)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("CreateIgnoreFile", "Add a ignore.conf file"))
				.ToolTipText(LOCTEXT("CreateIgnoreFile_Tooltip", "Create and add a standard 'ignore.conf' file"))
				.OnClicked(this, &SPlasticSourceControlSettings::OnClickedAddIgnoreFile)
				.HAlign(HAlign_Center)
			]
		]
	];
}

EVisibility SPlasticSourceControlSettings::PlasticNotAvailable() const
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	return Provider.IsPlasticAvailable() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SPlasticSourceControlSettings::GetBinaryPathText() const
{
	return FText::FromString(FPlasticSourceControlModule::Get().GetProvider().AccessSettings().GetBinaryPath());
}

void SPlasticSourceControlSettings::OnBinaryPathTextCommited(const FText& InText, ETextCommit::Type InCommitType) const
{
	FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const bool bChanged = Provider.AccessSettings().SetBinaryPath(InText.ToString());
	if (bChanged)
	{
		// Re-Check provided cm binary path for each change
		Provider.CheckPlasticAvailability();
		if (Provider.IsPlasticAvailable())
		{
			Provider.AccessSettings().SaveSettings();
		}
	}
}

FText SPlasticSourceControlSettings::GetVersions() const
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	return FText::FromString(Provider.GetPlasticScmVersion().String + TEXT("\t(plugin v") + Provider.GetPluginVersion() + TEXT(")"));
}

FText SPlasticSourceControlSettings::GetPathToWorkspaceRoot() const
{
	return FText::FromString(FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot());
}

FText SPlasticSourceControlSettings::GetUserName() const
{
	return FText::FromString(FPlasticSourceControlModule::Get().GetProvider().GetUserName());
}


EVisibility SPlasticSourceControlSettings::CanCreatePlasticWorkspace() const
{
	const FPlasticSourceControlProvider& Provider = FPlasticSourceControlModule::Get().GetProvider();
	const bool bPlasticAvailable = Provider.IsPlasticAvailable();
	const bool bPlasticWorkspaceFound = Provider.IsWorkspaceFound();
	return (bPlasticAvailable && !bPlasticWorkspaceFound) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SPlasticSourceControlSettings::IsReadyToCreatePlasticWorkspace() const
{
	// Workspace Name cannot be left empty
	const bool bWorkspaceNameOk = !WorkspaceParams.WorkspaceName.IsEmpty();
	// RepositoryName and ServerUrl should also be filled
	const bool bRepositoryNameOk = !WorkspaceParams.RepositoryName.IsEmpty() && !WorkspaceParams.ServerUrl.IsEmpty();
	// If Initial Commit is requested, checkin message cannot be empty
	const bool bInitialCommitOk = (!WorkspaceParams.bAutoInitialCommit || !WorkspaceParams.InitialCommitMessage.IsEmpty());
	return bWorkspaceNameOk && bRepositoryNameOk && bInitialCommitOk;
}


void SPlasticSourceControlSettings::OnWorkspaceNameCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	WorkspaceParams.WorkspaceName = InText;
}
FText SPlasticSourceControlSettings::GetWorkspaceName() const
{
	return WorkspaceParams.WorkspaceName;
}

void SPlasticSourceControlSettings::OnRepositoryNameCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	WorkspaceParams.RepositoryName = InText;
}
FText SPlasticSourceControlSettings::GetRepositoryName() const
{
	return WorkspaceParams.RepositoryName;
}

void SPlasticSourceControlSettings::OnServerUrlCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	WorkspaceParams.ServerUrl = InText;
}
FText SPlasticSourceControlSettings::GetServerUrl() const
{
	return WorkspaceParams.ServerUrl;
}

bool SPlasticSourceControlSettings::CreatePartialWorkspace() const
{
	return WorkspaceParams.bCreatePartialWorkspace;
}

void SPlasticSourceControlSettings::OnCheckedCreatePartialWorkspace(ECheckBoxState NewCheckedState)
{
	WorkspaceParams.bCreatePartialWorkspace = (NewCheckedState == ECheckBoxState::Checked);
}

bool SPlasticSourceControlSettings::CanAutoCreateIgnoreFile() const
{
	const bool bIgnoreFileFound = FPaths::FileExists(GetIgnoreFileName());
	return !bIgnoreFileFound;
}

void SPlasticSourceControlSettings::OnCheckedCreateIgnoreFile(ECheckBoxState NewCheckedState)
{
	bAutoCreateIgnoreFile = (NewCheckedState == ECheckBoxState::Checked);
}

void SPlasticSourceControlSettings::OnCheckedInitialCommit(ECheckBoxState NewCheckedState)
{
	WorkspaceParams.bAutoInitialCommit = (NewCheckedState == ECheckBoxState::Checked);
}

void SPlasticSourceControlSettings::OnInitialCommitMessageCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	WorkspaceParams.InitialCommitMessage = InText;
}

FText SPlasticSourceControlSettings::GetInitialCommitMessage() const
{
	return WorkspaceParams.InitialCommitMessage;
}


FReply SPlasticSourceControlSettings::OnClickedCreatePlasticWorkspace()
{
	UE_LOGF(LogSourceControl, Log, "CreatePlasticWorkspace(%ls, %ls, %ls) PartialWorkspace=%d CreateIgnore=%d Commit=%d",
		*WorkspaceParams.WorkspaceName.ToString(), *WorkspaceParams.RepositoryName.ToString(), *WorkspaceParams.ServerUrl.ToString(),
		WorkspaceParams.bCreatePartialWorkspace, bAutoCreateIgnoreFile, WorkspaceParams.bAutoInitialCommit);

	if (bAutoCreateIgnoreFile)
	{
		// 1. Create a standard "ignore.conf" file with common patterns for a typical Blueprint & C++ project
		CreateIgnoreFile();
	}

	// 2. Create a repository (if not already existing) and a workspace: launch an asynchronous MakeWorkspace operation
	FPlasticSourceControlModule::Get().GetWorkspaceCreation().MakeWorkspace(WorkspaceParams);

	return FReply::Handled();
}

/** Delegate to check for presence of an ignore.conf file to an existing Unity Version Control workspace */
EVisibility SPlasticSourceControlSettings::CanAddIgnoreFile() const
{
	const bool bPlasticWorkspaceFound = FPlasticSourceControlModule::Get().GetProvider().IsWorkspaceFound();
	const bool bIgnoreFileFound = FPaths::FileExists(GetIgnoreFileName());
	return (bPlasticWorkspaceFound && !bIgnoreFileFound) ? EVisibility::Visible : EVisibility::Collapsed;
}

/** Delegate to add an ignore.conf file to an existing Unity Version Control workspace */
FReply SPlasticSourceControlSettings::OnClickedAddIgnoreFile() const
{
	if (CreateIgnoreFile())
	{
		// Add ignore.conf to Unity Version Control
		TArray<FString> InfoMessages;
		TArray<FString> ErrorMessages;
		TArray<FString> Parameters;
		Parameters.Add(TEXT("-R"));
		TArray<FString> Files;
		Files.Add(TEXT("ignore.conf"));
		PlasticSourceControlUtils::RunCommand(TEXT("add"), Parameters, Files, InfoMessages, ErrorMessages);
	}
	return FReply::Handled();
}

void SPlasticSourceControlSettings::OnCheckedUpdateStatusAtStartup(ECheckBoxState NewCheckedState)
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	PlasticSettings.SetUpdateStatusAtStartup(NewCheckedState == ECheckBoxState::Checked);
	PlasticSettings.SaveSettings();
}

ECheckBoxState SPlasticSourceControlSettings::IsUpdateStatusAtStartupChecked() const
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	return PlasticSettings.GetUpdateStatusAtStartup() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPlasticSourceControlSettings::OnCheckedUpdateStatusOtherBranches(ECheckBoxState NewCheckedState)
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	PlasticSettings.SetUpdateStatusOtherBranches(NewCheckedState == ECheckBoxState::Checked);
	PlasticSettings.SaveSettings();
}

ECheckBoxState SPlasticSourceControlSettings::IsUpdateStatusOtherBranchesChecked() const
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	return PlasticSettings.GetUpdateStatusOtherBranches() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPlasticSourceControlSettings::OnCheckedViewLocalChanges(ECheckBoxState NewCheckedState)
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	PlasticSettings.SetViewLocalChanges(NewCheckedState == ECheckBoxState::Checked);
	PlasticSettings.SaveSettings();
}

ECheckBoxState SPlasticSourceControlSettings::IsViewLocalChangesChecked() const
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	return PlasticSettings.GetViewLocalChanges() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPlasticSourceControlSettings::OnCheckedEnableVerboseLogs(ECheckBoxState NewCheckedState)
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	PlasticSettings.SetEnableVerboseLogs(NewCheckedState == ECheckBoxState::Checked);
	PlasticSettings.SaveSettings();

	PlasticSourceControlUtils::SwitchVerboseLogs(NewCheckedState == ECheckBoxState::Checked);
}

ECheckBoxState SPlasticSourceControlSettings::IsEnableVerboseLogsChecked() const
{
	FPlasticSourceControlSettings& PlasticSettings = FPlasticSourceControlModule::Get().GetProvider().AccessSettings();
	return PlasticSettings.GetEnableVerboseLogs() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

/** Path to the "ignore.conf" file */
const FString SPlasticSourceControlSettings::GetIgnoreFileName() const
{
	const FString PathToWorkspaceRoot = FPlasticSourceControlModule::Get().GetProvider().GetPathToWorkspaceRoot();
	const FString IgnoreFileName = FPaths::Combine(*PathToWorkspaceRoot, TEXT("ignore.conf"));
	return IgnoreFileName;
}

/** Create a standard "ignore.conf" file with common patterns for a typical Blueprint & C++ project */
bool SPlasticSourceControlSettings::CreateIgnoreFile() const
{
	const FString IgnoreFileContent = TEXT("Binaries\nBuild\nDerivedDataCache\nIntermediate\nSaved\nScript\nenc_temp_folder\n.idea\n.vscode\n.vs\n.ignore\n*.VC.db\n*.opensdf\n*.opendb\n*.sdf\n*.sln\n*.suo\n*.code-workspace\n*.xcodeproj\n*.xcworkspace");
	return FFileHelper::SaveStringToFile(IgnoreFileContent, *GetIgnoreFileName(), FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
}

#undef LOCTEXT_NAMESPACE
