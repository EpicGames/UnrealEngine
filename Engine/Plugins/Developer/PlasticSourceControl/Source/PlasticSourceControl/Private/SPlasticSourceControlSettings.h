// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"

#include "PlasticSourceControlWorkspaceCreation.h"

class SPlasticSourceControlSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlSettings) {}

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

private:
	EVisibility PlasticNotAvailable() const;

	/** Delegate to get cm binary path from settings */
	FText GetBinaryPathText() const;

	/** Delegate to commit cm binary path to settings */
	void OnBinaryPathTextCommited(const FText& InText, ETextCommit::Type InCommitType) const;

	/** Delegate to get workspace root and user name from provider */
	FText GetVersions() const;
	FText GetPathToWorkspaceRoot() const;
	FText GetUserName() const;

	/** Delegate to create a new Plastic workspace and repository */
	EVisibility CanCreatePlasticWorkspace() const;
	bool IsReadyToCreatePlasticWorkspace() const;
	void OnWorkspaceNameCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetWorkspaceName() const;
	void OnRepositoryNameCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetRepositoryName() const;
	void OnServerUrlCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetServerUrl() const;
	bool CreatePartialWorkspace() const;
	void OnCheckedCreatePartialWorkspace(ECheckBoxState NewCheckedState);
	bool CanAutoCreateIgnoreFile() const;
	void OnCheckedCreateIgnoreFile(ECheckBoxState NewCheckedState);
	bool bAutoCreateIgnoreFile;

	void OnCheckedUpdateStatusAtStartup(ECheckBoxState NewCheckedState);
	ECheckBoxState IsUpdateStatusAtStartupChecked() const;

	void OnCheckedUpdateStatusOtherBranches(ECheckBoxState NewCheckedState);
	ECheckBoxState IsUpdateStatusOtherBranchesChecked() const;

	void OnCheckedViewLocalChanges(ECheckBoxState NewCheckedState);
	ECheckBoxState IsViewLocalChangesChecked() const;

	void OnCheckedEnableVerboseLogs(ECheckBoxState NewCheckedState);
	ECheckBoxState IsEnableVerboseLogsChecked() const;

	void OnCheckedInitialCommit(ECheckBoxState NewCheckedState);
	void OnInitialCommitMessageCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetInitialCommitMessage() const;

	FReply OnClickedCreatePlasticWorkspace();

	// Parameters for the workspace creation
	FPlasticSourceControlWorkspaceCreation::FParameters WorkspaceParams;

	/** Delegate to add a Plastic ignore.conf file to an existing Plastic workspace */
	EVisibility CanAddIgnoreFile() const;
	FReply OnClickedAddIgnoreFile() const;

	const FString GetIgnoreFileName() const;
	bool CreateIgnoreFile() const;
};
