// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlasticSourceControlStatusBar.h"

#include "PlasticSourceControlModule.h"

#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "PlasticSourceControl"

void SPlasticSourceControlStatusBar::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SButton)
		.ContentPadding(FMargin(6.0f, 0.0f))
		.ToolTipText(LOCTEXT("PlasticBranchesWindowTooltip", "Open the Branches window."))
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SPlasticSourceControlStatusBar::OnClicked)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("SourceControl.Branch"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
				.Text_Lambda([this]() { return GetStatusBarText(); })
			]
		]
	];
}

FText SPlasticSourceControlStatusBar::GetStatusBarText() const
{;
	return FText::FromString(FPlasticSourceControlModule::Get().GetProvider().GetWorkspaceSelector());
}

FReply SPlasticSourceControlStatusBar::OnClicked()
{
	FPlasticSourceControlModule::Get().GetBranchesWindow().OpenTab();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
