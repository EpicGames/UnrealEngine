// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"

class FReply;

/**
 * Status bar displaying the name of the current branch
 */
class SPlasticSourceControlStatusBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlasticSourceControlStatusBar)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FText GetStatusBarText() const;

	FReply OnClicked();
};
