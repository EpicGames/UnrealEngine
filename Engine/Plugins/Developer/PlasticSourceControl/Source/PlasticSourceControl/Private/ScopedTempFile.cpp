// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopedTempFile.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"

#include "ISourceControlModule.h" // LogSourceControl

FScopedTempFile::FScopedTempFile() :
	FScopedTempFile(TEXT("Temp-"), TEXT(".xml"))
{
}

FScopedTempFile::FScopedTempFile(const FString& InPrefix, const FString& InExtension)
{
	Filename = FPaths::CreateTempFilename(*FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir()), *InPrefix, *InExtension);
}

FScopedTempFile::FScopedTempFile(const FString& InText) :
	FScopedTempFile(TEXT("Temp-"), TEXT(".txt"))
{
	if (!FFileHelper::SaveStringToFile(InText, *Filename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogSourceControl, Error, "Failed to write to temp file: %ls", *Filename);
	}
}

FScopedTempFile::FScopedTempFile(const FText& InText) :
	FScopedTempFile(InText.ToString())
{
}

FScopedTempFile::~FScopedTempFile()
{
	if (FPaths::FileExists(Filename))
	{
		if (!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Filename))
		{
			UE_LOGF(LogSourceControl, Error, "Failed to delete temp file: %ls", *Filename);
		}
	}
}

const FString& FScopedTempFile::GetFilename() const
{
	return Filename;
}
