// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Helper for temporary files to pass as arguments to some commands (typically for checkin multi-line text message)
 */
class FScopedTempFile
{
public:
	/** Default constructor - only hold a temp filename */
	FScopedTempFile();
	FScopedTempFile(const FString& InPrefix, const FString& InExtension);

	/** Constructor - open & write string to temp file */
	explicit FScopedTempFile(const FString& InText);
	explicit FScopedTempFile(const FText& InText);

	/** Destructor - delete temp file */
	~FScopedTempFile();

	/** Get the filename of this temp file - empty if it failed to be created */
	const FString& GetFilename() const;

private:
	/** The filename we are writing to */
	FString Filename;
};
