// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftwareVersion.h"

FSoftwareVersion::FSoftwareVersion(FString&& InVersionString)
{
	String = MoveTemp(InVersionString);
	TArray<FString> VersionElements;
	String.ParseIntoArray(VersionElements, TEXT("."));
	if (VersionElements.Num() == 4)
	{
		Major = FCString::Atoi(*VersionElements[0]);
		Minor = FCString::Atoi(*VersionElements[1]);
		Patch = FCString::Atoi(*VersionElements[2]);
		Changeset = FCString::Atoi(*VersionElements[3]);
	}
}

bool operator==(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs)
{
	return (Rhs.Major == Lhs.Major) && (Rhs.Minor == Lhs.Minor) && (Rhs.Patch == Lhs.Patch) && (Rhs.Changeset == Lhs.Changeset);
}

bool operator<(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs)
{
	if (Rhs.Major < Lhs.Major) return true;
	if (Rhs.Major > Lhs.Major) return false;
	if (Rhs.Minor < Lhs.Minor) return true;
	if (Rhs.Minor > Lhs.Minor) return false;
	if (Rhs.Patch < Lhs.Patch) return true;
	if (Rhs.Patch > Lhs.Patch) return false;
	if (Rhs.Changeset < Lhs.Changeset) return true;
	if (Rhs.Changeset > Lhs.Changeset) return false;
	return false; // Equal
}

bool operator>=(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs)
{
	return !(Rhs < Lhs);
}
