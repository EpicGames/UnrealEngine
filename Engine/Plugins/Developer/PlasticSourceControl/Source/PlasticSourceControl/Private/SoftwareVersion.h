// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Software version string in the form "X.Y.Z.C", ie Major.Minor.Patch.Changeset (as returned by GetPlasticScmVersion)
*/
struct FSoftwareVersion
{
	FSoftwareVersion() : String(TEXT("<unknown-version>")) {}

	explicit FSoftwareVersion(FString&& InVersionString);

	FString String;

	int32 Major = 0;
	int32 Minor = 0;
	int32 Patch = 0;
	int32 Changeset = 0;
};

bool operator==(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs);
bool operator<(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs);
bool operator>=(const FSoftwareVersion& Rhs, const FSoftwareVersion& Lhs);
